{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Render Grid",
  "Description": "Advanced pipeline for use in creating rendered cinematics.",
  "Category": "Experimental",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "",
  "MarketplaceURL": "",
  "SupportURL": "",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": false,
  "IsExperimentalVersion": true,
  "Installed": false,
  "Modules": [
    {
      "Name": "RenderGrid",
      "Type": "Editor",
      "LoadingPhase": "PostEngineInit"
    },
    {
      "Name": "RenderGridDeveloper",
      "Type": "Editor",
      "LoadingPhase": "PostEngineInit"
    },
    {
      "Name": "RenderGridEditor",
      "Type": "Editor",
      "LoadingPhase": "PostEngineInit"
    }
  ],
  "Plugins": [
    {
      "Name": "RemoteControl",
      "Enabled": true,
      "Optional": false
    },
    {
      "Name": "LevelSequenceEditor",
      "Enabled": true,
      "Optional": false
    },
    {
      "Name": "MovieRenderPipeline",
      "Enabled": true,
      "Optional": false
    }
  ]
}