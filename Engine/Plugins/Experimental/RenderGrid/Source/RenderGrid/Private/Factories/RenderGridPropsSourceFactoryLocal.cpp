// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RenderGridPropsSourceFactoryLocal.h"
#include "RenderGrid/RenderGridPropsSource.h"


URenderGridPropsSourceBase* UE::RenderGrid::Private::FRenderGridPropsSourceFactoryLocal::CreateInstance(UObject* PropsSourceOrigin)
{
	URenderGridPropsSourceLocal* PropsSource = NewObject<URenderGridPropsSourceLocal>();
	PropsSource->SetSourceOrigin(PropsSourceOrigin);
	return PropsSource;
}
