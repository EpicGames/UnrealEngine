// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/IRenderGridPropsSourceFactory.h"


namespace UE::RenderGrid::Private
{
	/**
	 * The factory class for URenderGridPropsSourceLocal.
	 */
	class RENDERGRID_API FRenderGridPropsSourceFactoryLocal final : public IRenderGridPropsSourceFactory
	{
	public:
		virtual URenderGridPropsSourceBase* CreateInstance(UObject* PropsSourceOrigin) override;
	};
}
