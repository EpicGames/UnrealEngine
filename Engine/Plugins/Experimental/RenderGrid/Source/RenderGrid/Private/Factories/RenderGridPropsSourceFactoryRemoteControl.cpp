// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RenderGridPropsSourceFactoryRemoteControl.h"
#include "RenderGrid/RenderGridPropsSource.h"


URenderGridPropsSourceBase* UE::RenderGrid::Private::FRenderGridPropsSourceFactoryRemoteControl::CreateInstance(UObject* PropsSourceOrigin)
{
	URenderGridPropsSourceRemoteControl* PropsSource = NewObject<URenderGridPropsSourceRemoteControl>();
	PropsSource->SetSourceOrigin(PropsSourceOrigin);
	return PropsSource;
}
