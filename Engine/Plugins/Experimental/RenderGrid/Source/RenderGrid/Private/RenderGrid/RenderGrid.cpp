// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGrid/RenderGrid.h"

#include "IRenderGridModule.h"
#include "LevelSequence.h"
#include "Misc/Base64.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MovieScene.h"
#include "Dom/JsonObject.h"
#include "RenderGrid/RenderGridManager.h"
#include "RenderGrid/RenderGridQueue.h"
#include "Utils/RenderGridUtils.h"
#include "UObject/ObjectSaveContext.h"


URenderGridSettings::URenderGridSettings()
	: PropsSourceType(ERenderGridPropsSourceType::RemoteControl)
	, PropsSourceOrigin_RemoteControl(nullptr)
	, CachedPropsSource(nullptr)
	, CachedPropsSourceType(ERenderGridPropsSourceType::Local)
	, CachedPropsSourceOriginWeakPtr(nullptr)
{
	SetFlags(RF_Public | RF_Transactional);
}


URenderGridDefaults::URenderGridDefaults()
	: LevelSequence(nullptr)
	, RenderPreset(nullptr)
	, OutputDirectory(UE::RenderGrid::Private::FRenderGridUtils::NormalizeJobOutputDirectory(FPaths::ProjectDir() / TEXT("Saved/MovieRenders/")))
{
	SetFlags(RF_Public | RF_Transactional);
}


URenderGridJob::URenderGridJob()
	: Guid(FGuid::NewGuid())
	, WaitFramesBeforeRendering(0)
	, LevelSequence(nullptr)
	, bOverrideStartFrame(false)
	, CustomStartFrame(0)
	, bOverrideEndFrame(false)
	, CustomEndFrame(0)
	, bOverrideResolution(false)
	, CustomResolution(FIntPoint(3840, 2160))
	, bIsEnabled(true)
	, RenderPreset(nullptr)
{
	SetFlags(RF_Public | RF_Transactional);
}

FString URenderGridJob::ToDebugString() const
{
	return UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(ToDebugJson(), true);
}

TSharedPtr<FJsonValue> URenderGridJob::ToDebugJson() const
{
	TSharedPtr<FJsonObject> Root = MakeShareable(new FJsonObject());
	{
		Root->SetStringField(TEXT("Class"), GetParentNativeClass(GetClass())->GetName());
		Root->SetStringField(TEXT("Object"), GetName());
		Root->SetStringField(TEXT("Guid"), GetGuid().ToString());
		Root->SetStringField(TEXT("JobId"), GetJobId());
		Root->SetStringField(TEXT("JobName"), GetJobName());
		Root->SetBoolField(TEXT("IsEnabled"), GetIsEnabled());
		Root->SetStringField(TEXT("RenderPreset"), GetPathNameSafe(GetRenderPreset()));
		Root->SetStringField(TEXT("OutputDirectory"), GetOutputDirectory());

		{
			const FIntPoint Value = GetOutputResolution();
			TSharedPtr<FJsonObject> Object = MakeShareable(new FJsonObject());
			Object->SetNumberField(TEXT("X"), Value.X);
			Object->SetNumberField(TEXT("Y"), Value.Y);
			Root->SetObjectField(TEXT("OutputResolution"), Object);
		}

		if (const TOptional<int32> Value = GetStartFrame(); Value.IsSet())
		{
			Root->SetNumberField(TEXT("StartFrame"), Value.Get(0));
		}
		if (const TOptional<int32> Value = GetEndFrame(); Value.IsSet())
		{
			Root->SetNumberField(TEXT("EndFrame"), Value.Get(0));
		}
		if (const TOptional<double> Value = GetStartTime(); Value.IsSet())
		{
			Root->SetNumberField(TEXT("StartTime"), Value.Get(0));
		}
		if (const TOptional<double> Value = GetEndTime(); Value.IsSet())
		{
			Root->SetNumberField(TEXT("EndTime"), Value.Get(0));
		}
		if (const TOptional<double> Value = GetDuration(); Value.IsSet())
		{
			Root->SetNumberField(TEXT("Duration"), Value.Get(0));
		}

		Root->SetNumberField(TEXT("WaitFramesBeforeRendering"), GetWaitFramesBeforeRendering());
		Root->SetStringField(TEXT("LevelSequence"), GetPathNameSafe(GetLevelSequence()));

		{
			TArray<TSharedPtr<FJsonValue>> Array;
			for (const TTuple<FGuid, FString>& Entry : GetRemoteControlValues())
			{
				TSharedPtr<FJsonObject> Object = MakeShareable(new FJsonObject());
				Object->SetStringField(TEXT("Guid"), Entry.Key.ToString());
				if (FString Label; GetRemoteControlLabelFromFieldId(Entry.Key, Label))
				{
					Object->SetStringField(TEXT("Label"), Label);
				}
				Object->SetField(TEXT("Value"), UE::RenderGrid::Private::FRenderGridUtils::ParseJson(Entry.Value));
				Array.Add(MakeShareable(new FJsonValueObject(Object)));
			}
			Root->SetArrayField(TEXT("RemoteControlValues"), Array);
		}
	}
	return MakeShareable(new FJsonValueObject(Root));
}

TOptional<int32> URenderGridJob::GetSequenceStartFrame() const
{
	if (!IsValid(LevelSequence))
	{
		return TOptional<int32>();
	}

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();

	if (bOverrideStartFrame || (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomPlaybackRange))
	{
		int32 Frame = (bOverrideStartFrame ? CustomStartFrame : MovieOutputSettings->CustomStartFrame);
		if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
		{
			return FMath::FloorToInt(Frame / (MovieOutputSettings->OutputFrameRate / DisplayRate).AsDecimal());
		}
		return Frame;
	}

	const int32 StartFrameNumber = MovieScene->GetPlaybackRange().GetLowerBoundValue().Value;
	return FMath::FloorToInt(StartFrameNumber / (TickResolution / DisplayRate).AsDecimal());
}

TOptional<int32> URenderGridJob::GetSequenceEndFrame() const
{
	if (!IsValid(LevelSequence))
	{
		return TOptional<int32>();
	}

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();

	if (bOverrideEndFrame || (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomPlaybackRange))
	{
		int32 Frame = (bOverrideEndFrame ? CustomEndFrame : MovieOutputSettings->CustomEndFrame);
		if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
		{
			return FMath::FloorToInt(Frame / (MovieOutputSettings->OutputFrameRate / DisplayRate).AsDecimal());
		}
		return Frame;
	}

	const int32 EndFrameNumber = MovieScene->GetPlaybackRange().GetUpperBoundValue().Value;
	return FMath::FloorToInt(EndFrameNumber / (TickResolution / DisplayRate).AsDecimal());
}

bool URenderGridJob::SetSequenceStartFrame(const int32 NewCustomStartFrame)
{
	int32 StartFrame = GetStartFrame().Get(0);
	SetIsUsingCustomStartFrame(true);
	SetCustomStartFrame(StartFrame);

	TOptional<int32> SequenceStartFrame = GetSequenceStartFrame();
	while (SequenceStartFrame.IsSet() && (SequenceStartFrame.Get(0) > NewCustomStartFrame))
	{
		StartFrame--;
		if (StartFrame <= INT32_MIN)
		{
			return false;
		}
		SetCustomStartFrame(StartFrame);
		SequenceStartFrame = GetSequenceStartFrame();
	}
	while (SequenceStartFrame.IsSet() && (SequenceStartFrame.Get(0) < NewCustomStartFrame))
	{
		StartFrame++;
		if (StartFrame >= INT32_MAX)
		{
			return false;
		}
		SetCustomStartFrame(StartFrame);
		SequenceStartFrame = GetSequenceStartFrame();
	}
	return (SequenceStartFrame.IsSet() && (SequenceStartFrame.Get(0) == NewCustomStartFrame));
}

bool URenderGridJob::SetSequenceEndFrame(const int32 NewCustomStartFrame)
{
	int32 EndFrame = GetEndFrame().Get(0);
	SetIsUsingCustomEndFrame(true);
	SetCustomEndFrame(EndFrame);

	TOptional<int32> SequenceEndFrame = GetSequenceEndFrame();
	while (SequenceEndFrame.IsSet() && (SequenceEndFrame.Get(0) > NewCustomStartFrame))
	{
		EndFrame--;
		if (EndFrame <= INT32_MIN)
		{
			return false;
		}
		SetCustomEndFrame(EndFrame);
		SequenceEndFrame = GetSequenceEndFrame();
	}
	while (SequenceEndFrame.IsSet() && (SequenceEndFrame.Get(0) < NewCustomStartFrame))
	{
		EndFrame++;
		if (EndFrame >= INT32_MAX)
		{
			return false;
		}
		SetCustomEndFrame(EndFrame);
		SequenceEndFrame = GetSequenceEndFrame();
	}
	return (SequenceEndFrame.IsSet() && (SequenceEndFrame.Get(0) == NewCustomStartFrame));
}

TOptional<int32> URenderGridJob::GetStartFrame() const
{
	if (bOverrideStartFrame)
	{
		return CustomStartFrame;
	}

	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomPlaybackRange)
	{
		return MovieOutputSettings->CustomStartFrame;
	}

	if (!IsValid(LevelSequence))
	{
		return TOptional<int32>();
	}

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
	{
		DisplayRate = MovieOutputSettings->OutputFrameRate;
	}
	const int32 StartFrameNumber = MovieScene->GetPlaybackRange().GetLowerBoundValue().Value;
	return FMath::FloorToInt(StartFrameNumber / (TickResolution / DisplayRate).AsDecimal());
}

TOptional<int32> URenderGridJob::GetEndFrame() const
{
	if (bOverrideEndFrame)
	{
		return CustomEndFrame;
	}

	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomPlaybackRange)
	{
		return MovieOutputSettings->CustomEndFrame;
	}

	if (!IsValid(LevelSequence))
	{
		return TOptional<int32>();
	}

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
	{
		DisplayRate = MovieOutputSettings->OutputFrameRate;
	}
	const int32 EndFrameNumber = MovieScene->GetPlaybackRange().GetUpperBoundValue().Value;
	return FMath::FloorToInt(EndFrameNumber / (TickResolution / DisplayRate).AsDecimal());
}

TOptional<double> URenderGridJob::GetStartTime() const
{
	if (!IsValid(LevelSequence))
	{
		return TOptional<double>();
	}

	const TOptional<int32> StartFrame = GetStartFrame();
	if (!StartFrame.IsSet())
	{
		return TOptional<double>();
	}

	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
	{
		DisplayRate = MovieOutputSettings->OutputFrameRate;
	}
	return StartFrame.Get(0) / DisplayRate.AsDecimal();
}

TOptional<double> URenderGridJob::GetEndTime() const
{
	if (!IsValid(LevelSequence))
	{
		return TOptional<double>();
	}

	const TOptional<int32> EndFrame = GetEndFrame();
	if (!EndFrame.IsSet())
	{
		return TOptional<double>();
	}

	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
	{
		DisplayRate = MovieOutputSettings->OutputFrameRate;
	}
	return EndFrame.Get(0) / DisplayRate.AsDecimal();
}

TOptional<double> URenderGridJob::GetDuration() const
{
	if (!IsValid(LevelSequence))
	{
		return TOptional<double>();
	}

	const TOptional<int32> StartFrame = GetStartFrame();
	const TOptional<int32> EndFrame = GetEndFrame();
	if (!StartFrame.IsSet() || !EndFrame.IsSet() || (StartFrame.Get(0) > EndFrame.Get(0)))
	{
		return TOptional<double>();
	}

	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();

	const UMovieScene* MovieScene = LevelSequence->MovieScene;
	FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	if (IsValid(MovieOutputSettings) && MovieOutputSettings->bUseCustomFrameRate)
	{
		DisplayRate = MovieOutputSettings->OutputFrameRate;
	}
	return (EndFrame.Get(0) - StartFrame.Get(0)) / DisplayRate.AsDecimal();
}

void URenderGridJob::GetStartFrame(bool& bSuccess, int32& StartFrame) const
{
	TOptional<int32> Result = GetStartFrame();
	bSuccess = Result.IsSet();
	StartFrame = Result.Get(0);
}

void URenderGridJob::GetEndFrame(bool& bSuccess, int32& EndFrame) const
{
	TOptional<int32> Result = GetEndFrame();
	bSuccess = Result.IsSet();
	EndFrame = Result.Get(0);
}

void URenderGridJob::GetStartTime(bool& bSuccess, double& StartTime) const
{
	TOptional<double> Result = GetStartTime();
	bSuccess = Result.IsSet();
	StartTime = Result.Get(0);
}

void URenderGridJob::GetEndTime(bool& bSuccess, double& EndTime) const
{
	TOptional<double> Result = GetEndTime();
	bSuccess = Result.IsSet();
	EndTime = Result.Get(0);
}

void URenderGridJob::GetDuration(bool& bSuccess, double& Duration) const
{
	TOptional<double> Result = GetDuration();
	bSuccess = Result.IsSet();
	Duration = Result.Get(0);
}

FIntPoint URenderGridJob::GetOutputResolution() const
{
	if (bOverrideResolution)
	{
		return CustomResolution;
	}
	if (UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings())
	{
		return MovieOutputSettings->OutputResolution;
	}
	return GetDefault<UMoviePipelineOutputSetting>()->OutputResolution;
}

double URenderGridJob::GetOutputAspectRatio() const
{
	const UMoviePipelineOutputSetting* MovieOutputSettings = GetRenderPresetOutputSettings();
	if (IsValid(MovieOutputSettings))
	{
		return static_cast<double>(MovieOutputSettings->OutputResolution.X) / static_cast<double>(MovieOutputSettings->OutputResolution.Y);
	}

	const UMoviePipelineOutputSetting* DefaultSettings = GetDefault<UMoviePipelineOutputSetting>();
	return static_cast<double>(DefaultSettings->OutputResolution.X) / static_cast<double>(DefaultSettings->OutputResolution.Y);
}

bool URenderGridJob::MatchesSearchTerm(const FString& SearchTerm) const
{
	if (SearchTerm.TrimStartAndEnd().Len() <= 0)
	{
		return true;
	}
	TArray<FString> Parts;
	SearchTerm.ParseIntoArray(Parts, TEXT(" "), true);
	for (const FString& Part : Parts)
	{
		FString TrimmedPart = Part.TrimStartAndEnd();
		if (TrimmedPart.Len() <= 0)
		{
			continue;
		}
		if ((JobId.Find(TrimmedPart) != INDEX_NONE) || (JobName.Find(TrimmedPart) != INDEX_NONE) || (OutputDirectory.Find(TrimmedPart) != INDEX_NONE))
		{
			continue;
		}
		if (IsValid(RenderPreset) && (RenderPreset.GetPath().Find(TrimmedPart) != INDEX_NONE))
		{
			continue;
		}
		return false;
	}
	return true;
}

void URenderGridJob::SetJobId(const FString& NewJobId)
{
	JobId = UE::RenderGrid::Private::FRenderGridUtils::PurgeJobId(NewJobId);
}

FString URenderGridJob::GetOutputDirectory() const
{
	return UE::RenderGrid::Private::FRenderGridUtils::DenormalizeJobOutputDirectory(OutputDirectory);
}

void URenderGridJob::SetOutputDirectory(const FString& NewOutputDirectory)
{
	OutputDirectory = UE::RenderGrid::Private::FRenderGridUtils::NormalizeJobOutputDirectory(NewOutputDirectory);
}

UMoviePipelineOutputSetting* URenderGridJob::GetRenderPresetOutputSettings() const
{
	if (!IsValid(RenderPreset))
	{
		return nullptr;
	}
	for (UMoviePipelineSetting* MovieSettings : RenderPreset->FindSettingsByClass(UMoviePipelineOutputSetting::StaticClass(), false))
	{
		if (!IsValid(MovieSettings))
		{
			continue;
		}
		if (UMoviePipelineOutputSetting* MovieOutputSettings = Cast<UMoviePipelineOutputSetting>(MovieSettings))
		{
			if (!MovieOutputSettings->IsEnabled())
			{
				continue;
			}
			return MovieOutputSettings;
		}
	}
	return nullptr;
}

TArray<URemoteControlPreset*> URenderGridJob::GetRemoteControlPresets() const
{
	TArray<URemoteControlPreset*> Presets;
	if (URenderGrid* Grid = GetTypedOuter<URenderGrid>(); IsValid(Grid))
	{
		if (URenderGridPropsSourceRemoteControl* PropsSource = Grid->GetPropsSource<URenderGridPropsSourceRemoteControl>(); IsValid(PropsSource))
		{
			if (URemoteControlPreset* Preset = PropsSource->GetRemoteControlPreset(); IsValid(Preset))
			{
				Presets.Add(Preset);
			}
		}
	}
	/*else
	{
		TArray<TSoftObjectPtr<URemoteControlPreset>> RemoteControlPresets;
		IRemoteControlModule::Get().GetPresets(RemoteControlPresets);
		for (TSoftObjectPtr<URemoteControlPreset> RemoteControlPresetWeakPtr : RemoteControlPresets)
		{
			if (URemoteControlPreset* Preset = RemoteControlPresetWeakPtr.Get(); IsValid(Preset))
			{
				Presets.Add(Preset);
			}
		}
	}*/
	return Presets;
}

bool URenderGridJob::HasStoredRemoteControlValueBytes(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity) const
{
	if (!RemoteControlEntity.IsValid())
	{
		return false;
	}

	return HasStoredRemoteControlValueBytes(RemoteControlEntity->GetId());
}

bool URenderGridJob::GetRemoteControlValueBytes(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, TArray<uint8>& OutBytes) const
{
	OutBytes.Empty();
	if (!RemoteControlEntity.IsValid())
	{
		return false;
	}

	if (FRenderGridRemoteControlPropertyData* DataPtr = RemoteControlValues.Find(RemoteControlEntity->GetId()))
	{
		OutBytes.Append((*DataPtr).Bytes);
		return true;
	}

	if (!URenderGridPropRemoteControl::GetValueOfEntity(RemoteControlEntity, OutBytes))
	{
		return false;
	}
	RemoteControlValues.Add(RemoteControlEntity->GetId(), FRenderGridRemoteControlPropertyData(TArray(OutBytes)));
	return true;
}

bool URenderGridJob::SetRemoteControlValueBytes(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes)
{
	if (!RemoteControlEntity.IsValid())
	{
		return false;
	}

	return SetRemoteControlValueBytes(RemoteControlEntity->GetId(), Bytes);
}

bool URenderGridJob::HasStoredRemoteControlValueBytes(const FGuid& FieldId) const
{
	if (!FieldId.IsValid())
	{
		return false;
	}

	return !!RemoteControlValues.Find(FieldId);
}

bool URenderGridJob::GetRemoteControlValueBytes(const FGuid& FieldId, TArray<uint8>& OutBytes) const
{
	OutBytes.Empty();
	if (!FieldId.IsValid())
	{
		return false;
	}

	if (FRenderGridRemoteControlPropertyData* DataPtr = RemoteControlValues.Find(FieldId))
	{
		OutBytes.Append((*DataPtr).Bytes);
		return true;
	}

	TSharedPtr<FRemoteControlEntity> RemoteControlEntity;
	for (URemoteControlPreset* RemoteControlPreset : GetRemoteControlPresets())
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				if (Prop->GetId() == FieldId)
				{
					RemoteControlEntity = Prop;
					break;
				}
			}
		}
	}

	if (!RemoteControlEntity.IsValid() || !URenderGridPropRemoteControl::GetValueOfEntity(RemoteControlEntity, OutBytes))
	{
		return false;
	}
	RemoteControlValues.Add(FieldId, FRenderGridRemoteControlPropertyData(TArray(OutBytes)));
	return true;
}

bool URenderGridJob::SetRemoteControlValueBytes(const FGuid& FieldId, const TArray<uint8>& Bytes)
{
	if (!FieldId.IsValid())
	{
		return false;
	}

	RemoteControlValues.Add(FieldId, FRenderGridRemoteControlPropertyData(TArray(Bytes)));
	return true;
}

bool URenderGridJob::GetRemoteControlValue(const FGuid& FieldId, FString& Json) const
{
	Json.Empty();
	TArray<uint8> Bytes;
	if (!GetRemoteControlValueBytes(FieldId, Bytes))
	{
		return false;
	}
	Json.Append(UE::RenderGrid::Private::FRenderGridUtils::GetRemoteControlValueJsonFromBytes(Bytes));
	return true;
}

bool URenderGridJob::SetRemoteControlValue(const FGuid& FieldId, const FString& Json)
{
	TArray<uint8> Bytes;
	if (!GetRemoteControlValueBytes(FieldId, Bytes))
	{
		return false;
	}
	return SetRemoteControlValueBytes(FieldId, UE::RenderGrid::Private::FRenderGridUtils::GetRemoteControlValueBytesFromJson(Bytes, Json));
}

bool URenderGridJob::GetRemoteControlFieldIdFromLabel(const FString& Label, FGuid& FieldId) const
{
	FieldId.Invalidate();
	const FName LabelName = FName(Label);
	for (URemoteControlPreset* RemoteControlPreset : GetRemoteControlPresets())
	{
		if (const FGuid RemoteControlFieldId = RemoteControlPreset->GetExposedEntityId(LabelName); RemoteControlFieldId.IsValid())
		{
			FieldId = RemoteControlFieldId;
			return true;
		}
	}
	return false;
}

bool URenderGridJob::GetRemoteControlLabelFromFieldId(const FGuid& FieldId, FString& Label) const
{
	Label.Empty();
	for (URemoteControlPreset* RemoteControlPreset : GetRemoteControlPresets())
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				if (Prop->GetId() == FieldId)
				{
					Label = Prop->GetLabel().ToString();
					return true;
				}
			}
		}
	}
	return false;
}

TMap<FGuid, FString> URenderGridJob::GetRemoteControlValues() const
{
	TMap<FGuid, FString> Result;
	for (URemoteControlPreset* RemoteControlPreset : GetRemoteControlPresets())
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				if (FString Json; GetRemoteControlValue(Prop->GetId(), Json))
				{
					Result.Add(Prop->GetId(), Json);
				}
			}
		}
	}
	return Result;
}


URenderGrid::URenderGrid()
	: Guid(FGuid::NewGuid())
	, Settings(CreateDefaultSubobject<URenderGridSettings>(TEXT("RenderGridSettings")))
	, Defaults(CreateDefaultSubobject<URenderGridDefaults>(TEXT("RenderGridDefaults")))
	, bExecutingBlueprintEvent(false)
{
	SetFlags(RF_Public);
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_DefaultSubObject))
	{
		ClearFlags(RF_Transactional);
	}
}

FString URenderGrid::ToDebugString() const
{
	return UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(ToDebugJson(), true);
}

TSharedPtr<FJsonValue> URenderGrid::ToDebugJson() const
{
	TSharedPtr<FJsonObject> Root = MakeShareable(new FJsonObject());
	{
		Root->SetStringField(TEXT("Class"), GetParentNativeClass(GetClass())->GetName());
		Root->SetStringField(TEXT("Object"), GetName());

		{
			FString Path = GetPathNameSafe(HasAnyFlags(RF_ClassDefaultObject) ? this : GetClass()->GetDefaultObject(true));
			if (FString PathLeft, PathRight; Path.Split(TEXT("."), &PathLeft, &PathRight, ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromEnd))
			{
				FString Name = PathLeft;
				if (FString NameLeft, NameRight; Name.Split(TEXT("/"), &NameLeft, &NameRight, ESearchCase::Type::IgnoreCase, ESearchDir::Type::FromEnd))
				{
					Name = NameRight;
				}
				Path = PathLeft + TEXT(".") + Name;
			}
			Root->SetStringField(TEXT("Path"), Path);
		}

		Root->SetStringField(TEXT("Guid"), GetGuid().ToString());

		{
			TSharedPtr<FJsonObject> Object = MakeShareable(new FJsonObject());
			Object->SetStringField(TEXT("PropsSourceType"), UEnum::GetDisplayValueAsText(GetPropsSourceType()).ToString());
			Object->SetStringField(TEXT("PropsSourceOrigin"), GetPathNameSafe(GetPropsSourceOrigin()));
			Root->SetObjectField(TEXT("Settings"), Object);
		}

		{
			TSharedPtr<FJsonObject> Object = MakeShareable(new FJsonObject());
			Object->SetStringField(TEXT("LevelSequence"), GetPathNameSafe(GetDefaultLevelSequence()));
			Object->SetStringField(TEXT("RenderPreset"), GetPathNameSafe(GetDefaultRenderPreset()));
			Object->SetStringField(TEXT("OutputDirectory"), GetDefaultOutputDirectory());
			Object->SetStringField(TEXT("OutputDirectory"), GetDefaultOutputDirectory());
			Root->SetObjectField(TEXT("Defaults"), Object);
		}

		{
			TArray<TSharedPtr<FJsonValue>> Array;
			for (URenderGridJob* Job : GetRenderGridJobs())
			{
				Array.Add(Job->ToDebugJson());
			}
			Root->SetArrayField(TEXT("Jobs"), Array);
		}
	}
	return MakeShareable(new FJsonValueObject(Root));
}

UWorld* URenderGrid::GetWorld() const
{
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			return Context.World();
		}
	}

	if (IsValid(GWorld))
	{
		return GWorld;
	}

	if (UWorld* CachedWorld = CachedWorldWeakPtr.Get(); IsValid(CachedWorld))
	{
		return CachedWorld;
	}
	UObject* Outer = GetOuter(); // Could be a GameInstance, could be World, could also be a WidgetTree, so we're just going to follow the outer chain to find the world we're in.
	while (IsValid(Outer))
	{
		if (UWorld* World = Cast<UWorld>(Outer); IsValid(World))
		{
			CachedWorldWeakPtr = World;
			return World;
		}
		if (UWorld* World = Outer->GetWorld(); IsValid(World))
		{
			CachedWorldWeakPtr = World;
			return World;
		}
		Outer = Outer->GetOuter();
	}
	return nullptr;
}

void URenderGrid::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
}

void URenderGrid::PostLoad()
{
	Super::PostLoad();

	SetFlags(RF_Public);

	if (!IsValid(Settings))
	{
		Settings = NewObject<URenderGridSettings>(this, TEXT("RenderGridSettings"));
	}
	Settings->SetFlags(RF_Public | RF_Transactional);
	if (Settings->PropsSourceType == ERenderGridPropsSourceType::Local)
	{
		Settings->PropsSourceType = ERenderGridPropsSourceType::RemoteControl;
	}

	if (!IsValid(Defaults))
	{
		Defaults = NewObject<URenderGridDefaults>(this, TEXT("RenderGridDefaults"));
	}
	Defaults->SetFlags(RF_Public | RF_Transactional);

	if (!HasAnyFlags(RF_ClassDefaultObject | RF_DefaultSubObject))
	{
		ClearFlags(RF_Transactional);
	}
	for (const TObjectPtr<URenderGridJob>& Job : RenderGridJobs)
	{
		if (IsValid(Job))
		{
			Job->SetFlags(RF_Public | RF_Transactional);
		}
	}
}

void URenderGrid::CopyJobs(URenderGrid* From)
{
	if (!IsValid(From))
	{
		return;
	}
	SetRenderGridJobs(From->GetRenderGridJobs());
}

void URenderGrid::CopyAllPropertiesExceptJobs(URenderGrid* From)
{
	if (!IsValid(From))
	{
		return;
	}
	for (FProperty* Property = StaticClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient) && !Property->GetName().Equals("RenderGridJobs") && !Property->GetName().Equals("Defaults") && !Property->GetName().Equals("Settings"))// if not [Transient] and not "RenderGridJobs"/"Defaults"/"Settings"
		{
			Property->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(this), Property->ContainerPtrToValuePtr<void>(From));
		}
	}
	Defaults->CopyValuesFrom(From->GetDefaultsObject());
	Settings->CopyValuesFrom(From->GetSettingsObject());
}

void URenderGrid::CopyAllProperties(URenderGrid* From)
{
	if (!IsValid(From))
	{
		return;
	}
	CopyJobs(From);
	CopyAllPropertiesExceptJobs(From);
}

void URenderGrid::CopyAllUserVariables(URenderGrid* From)
{
	if (!IsValid(From))
	{
		return;
	}
	if (From->GetClass()->IsChildOf(GetClass()))
	{
		for (FProperty* Property = GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!StaticClass()->HasProperty(Property))// if is [Blueprint Graph Variable]
			{
				Property->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(this), Property->ContainerPtrToValuePtr<void>(From));
			}
		}
	}
}

void URenderGrid::BeginEditor()
{
	bExecutingBlueprintEvent = true;
	ReceiveBeginEditor();
	bExecutingBlueprintEvent = false;
}

void URenderGrid::EndEditor()
{
	bExecutingBlueprintEvent = true;
	ReceiveEndEditor();
	bExecutingBlueprintEvent = false;
}

void URenderGrid::Tick(float DeltaTime)
{
	bExecutingBlueprintEvent = true;
	ReceiveTick(DeltaTime);
	bExecutingBlueprintEvent = false;
}

void URenderGrid::BeginBatchRender(URenderGridQueue* Queue)
{
	bExecutingBlueprintEvent = true;
	ReceiveBeginBatchRender(Queue);
	bExecutingBlueprintEvent = false;
}

void URenderGrid::EndBatchRender(URenderGridQueue* Queue)
{
	bExecutingBlueprintEvent = true;
	ReceiveEndBatchRender(Queue);
	bExecutingBlueprintEvent = false;
}

void URenderGrid::BeginJobRender(URenderGridQueue* Queue, URenderGridJob* Job)
{
	bExecutingBlueprintEvent = true;
	ReceiveBeginJobRender(Queue, Job);
	bExecutingBlueprintEvent = false;
}

void URenderGrid::EndJobRender(URenderGridQueue* Queue, URenderGridJob* Job)
{
	bExecutingBlueprintEvent = true;
	ReceiveEndJobRender(Queue, Job);
	bExecutingBlueprintEvent = false;
}

void URenderGrid::BeginViewportRender(URenderGridJob* Job)
{
	bExecutingBlueprintEvent = true;
	ReceiveBeginViewportRender(Job);
	bExecutingBlueprintEvent = false;
}

void URenderGrid::EndViewportRender(URenderGridJob* Job)
{
	bExecutingBlueprintEvent = true;
	ReceiveEndViewportRender(Job);
	bExecutingBlueprintEvent = false;
}

URenderGridQueue* URenderGrid::RunRenderJobs(const TArray<URenderGridJob*>& Jobs, const FRenderGridRunRenderJobsDefaultObjectCallback& DefaultObjectCallback, const FRenderGridRunRenderJobsCallback& Callback)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (URenderGrid* DefaultObject = Cast<URenderGrid>(GetClass()->GetDefaultObject(true)); IsValid(DefaultObject))
		{
			CopyAllProperties(DefaultObject);
			CopyAllUserVariables(DefaultObject);
			if (DefaultObjectCallback.IsBound())
			{
				return DefaultObjectCallback.Execute(DefaultObject, Jobs);
			}
		}
		return nullptr;
	}

	TArray<URenderGridJob*> JobsToRender;
	for (URenderGridJob* Job : Jobs)
	{
		if (IsValid(Job) && HasRenderGridJob(Job))
		{
			JobsToRender.Add(Job);
		}
	}
	if (Callback.IsBound())
	{
		if (URenderGridQueue* RenderQueue = Callback.Execute(this, Jobs); IsValid(RenderQueue))
		{
			RenderQueue->Execute();
			return RenderQueue;
		}
	}
	return nullptr;
}

URenderGridQueue* URenderGrid::Render()
{
	return RenderJobs(GetEnabledRenderGridJobs());
}

URenderGridQueue* URenderGrid::RenderJob(URenderGridJob* Job)
{
	return RenderJobs({Job});
}

URenderGridQueue* URenderGrid::RenderJobs(const TArray<URenderGridJob*>& Jobs)
{
	return RunRenderJobs(
		Jobs,
		FRenderGridRunRenderJobsDefaultObjectCallback::CreateLambda([](URenderGrid* InDefaultObject, const TArray<URenderGridJob*>& InJobs) -> URenderGridQueue* {
			return InDefaultObject->RenderJobs(InJobs);
		}),
		FRenderGridRunRenderJobsCallback::CreateLambda([](URenderGrid* InThis, const TArray<URenderGridJob*>& InJobs) -> URenderGridQueue* {
			return UE::RenderGrid::IRenderGridModule::Get().GetManager().CreateBatchRenderQueue(InThis, InJobs);
		})
	);
}

URenderGridQueue* URenderGrid::RenderSingleFrame(const int32 Frame)
{
	return RenderJobsSingleFrame(GetEnabledRenderGridJobs(), Frame);
}

URenderGridQueue* URenderGrid::RenderJobSingleFrame(URenderGridJob* Job, const int32 Frame)
{
	return RenderJobsSingleFrame({Job}, Frame);
}

URenderGridQueue* URenderGrid::RenderJobsSingleFrame(const TArray<URenderGridJob*>& Jobs, const int32 Frame)
{
	return RunRenderJobs(
		Jobs,
		FRenderGridRunRenderJobsDefaultObjectCallback::CreateLambda([Frame](URenderGrid* InDefaultObject, const TArray<URenderGridJob*>& InJobs) -> URenderGridQueue* {
			return InDefaultObject->RenderJobsSingleFrame(InJobs, Frame);
		}),
		FRenderGridRunRenderJobsCallback::CreateLambda([Frame](URenderGrid* InThis, const TArray<URenderGridJob*>& InJobs) -> URenderGridQueue* {
			return UE::RenderGrid::IRenderGridModule::Get().GetManager().CreateBatchRenderQueueSingleFrame(InThis, InJobs, Frame);
		})
	);
}

URenderGridQueue* URenderGrid::RenderSingleFramePosition(const double FramePosition)
{
	return RenderJobsSingleFramePosition(GetEnabledRenderGridJobs(), FramePosition);
}

URenderGridQueue* URenderGrid::RenderJobSingleFramePosition(URenderGridJob* Job, const double FramePosition)
{
	return RenderJobsSingleFramePosition({Job}, FramePosition);
}

URenderGridQueue* URenderGrid::RenderJobsSingleFramePosition(const TArray<URenderGridJob*>& Jobs, const double FramePosition)
{
	return RunRenderJobs(
		Jobs,
		FRenderGridRunRenderJobsDefaultObjectCallback::CreateLambda([FramePosition](URenderGrid* InDefaultObject, const TArray<URenderGridJob*>& InJobs) -> URenderGridQueue* {
			return InDefaultObject->RenderJobsSingleFramePosition(InJobs, FramePosition);
		}),
		FRenderGridRunRenderJobsCallback::CreateLambda([FramePosition](URenderGrid* InThis, const TArray<URenderGridJob*>& InJobs) -> URenderGridQueue* {
			return UE::RenderGrid::IRenderGridModule::Get().GetManager().CreateBatchRenderQueueSingleFramePosition(InThis, InJobs, FramePosition);
		})
	);
}

TSoftObjectPtr<UWorld> URenderGrid::GetLevel() const
{
	return Settings->Level;
}

void URenderGrid::SetPropsSource(ERenderGridPropsSourceType InPropsSourceType, UObject* InPropsSourceOrigin)
{
	if (InPropsSourceType == ERenderGridPropsSourceType::RemoteControl)
	{
		Settings->PropsSourceType = InPropsSourceType;
		Settings->PropsSourceOrigin_RemoteControl = Cast<URemoteControlPreset>(InPropsSourceOrigin);
		return;
	}
	Settings->PropsSourceType = ERenderGridPropsSourceType::RemoteControl;
	Settings->PropsSourceOrigin_RemoteControl = nullptr;
}

URenderGridPropsSourceBase* URenderGrid::GetPropsSource() const
{
	UObject* PropsSourceOrigin = GetPropsSourceOrigin();
	if (!IsValid(Settings->CachedPropsSource) || (Settings->CachedPropsSourceType != Settings->PropsSourceType) || (Settings->CachedPropsSourceOriginWeakPtr.Get() != PropsSourceOrigin))
	{
		Settings->CachedPropsSourceType = Settings->PropsSourceType;
		Settings->CachedPropsSourceOriginWeakPtr = PropsSourceOrigin;
		Settings->CachedPropsSource = UE::RenderGrid::IRenderGridModule::Get().CreatePropsSource(Settings->PropsSourceType, PropsSourceOrigin);
	}
	return Settings->CachedPropsSource;
}

UObject* URenderGrid::GetPropsSourceOrigin() const
{
	if (Settings->PropsSourceType == ERenderGridPropsSourceType::RemoteControl)
	{
		return Settings->PropsSourceOrigin_RemoteControl;
	}
	return nullptr;
}

UMoviePipelineOutputSetting* URenderGrid::GetDefaultRenderPresetOutputSettings() const
{
	if (!IsValid(Defaults->RenderPreset))
	{
		return nullptr;
	}
	for (UMoviePipelineSetting* MovieSettings : Defaults->RenderPreset->FindSettingsByClass(UMoviePipelineOutputSetting::StaticClass(), false))
	{
		if (!IsValid(MovieSettings))
		{
			continue;
		}
		if (UMoviePipelineOutputSetting* MovieOutputSettings = Cast<UMoviePipelineOutputSetting>(MovieSettings))
		{
			if (!MovieOutputSettings->IsEnabled())
			{
				continue;
			}
			return MovieOutputSettings;
		}
	}
	return nullptr;
}

FString URenderGrid::GetDefaultOutputDirectory() const
{
	return UE::RenderGrid::Private::FRenderGridUtils::DenormalizeJobOutputDirectory(Defaults->OutputDirectory);
}

void URenderGrid::SetDefaultOutputDirectory(const FString& NewOutputDirectory)
{
	Defaults->OutputDirectory = UE::RenderGrid::Private::FRenderGridUtils::NormalizeJobOutputDirectory(NewOutputDirectory);
}

void URenderGrid::ClearRenderGridJobs()
{
	RenderGridJobs.Empty();
}

void URenderGrid::SetRenderGridJobs(const TArray<URenderGridJob*>& Jobs)
{
	RenderGridJobs.Empty();
	for (URenderGridJob* Job : Jobs)
	{
		AddRenderGridJob(Job);
	}
}

void URenderGrid::AddRenderGridJob(URenderGridJob* Job)
{
	if (IsValid(Job))
	{
		RenderGridJobs.Add(Job);
	}
}

void URenderGrid::RemoveRenderGridJob(URenderGridJob* Job)
{
	RenderGridJobs.Remove(Job);
}

void URenderGrid::InsertRenderGridJob(URenderGridJob* Job, int32 Index)
{
	if (IsValid(Job))
	{
		RenderGridJobs.Insert(Job, Index);
	}
}

bool URenderGrid::HasRenderGridJob(URenderGridJob* Job) const
{
	return (RenderGridJobs.Find(Job) != INDEX_NONE);
}

int32 URenderGrid::GetIndexOfRenderGridJob(URenderGridJob* Job) const
{
	return RenderGridJobs.Find(Job);
}

TArray<URenderGridJob*> URenderGrid::GetRenderGridJobs() const
{
	TArray<URenderGridJob*> Result;
	for (URenderGridJob* Job : RenderGridJobs)
	{
		if (!IsValid(Job))
		{
			continue;
		}
		Result.Add(Job);
	}
	return Result;
}

TArray<URenderGridJob*> URenderGrid::GetEnabledRenderGridJobs() const
{
	TArray<URenderGridJob*> Result;
	for (URenderGridJob* Job : RenderGridJobs)
	{
		if (!IsValid(Job))
		{
			continue;
		}
		if (Job->GetIsEnabled())
		{
			Result.Add(Job);
		}
	}
	return Result;
}

TArray<URenderGridJob*> URenderGrid::GetDisabledRenderGridJobs() const
{
	TArray<URenderGridJob*> Result;
	for (URenderGridJob* Job : RenderGridJobs)
	{
		if (!IsValid(Job))
		{
			continue;
		}
		if (!Job->GetIsEnabled())
		{
			Result.Add(Job);
		}
	}
	return Result;
}

void URenderGrid::InsertRenderGridJobBefore(URenderGridJob* Job, URenderGridJob* BeforeJob)
{
	if (IsValid(Job))
	{
		const TArray<TObjectPtr<URenderGridJob>>::SizeType Index = RenderGridJobs.Find(BeforeJob);
		if (Index == INDEX_NONE)
		{
			RenderGridJobs.Add(Job);
		}
		else
		{
			RenderGridJobs.Insert(Job, Index);
		}
	}
}

void URenderGrid::InsertRenderGridJobAfter(URenderGridJob* Job, URenderGridJob* AfterJob)
{
	if (IsValid(Job))
	{
		const TArray<TObjectPtr<URenderGridJob>>::SizeType Index = RenderGridJobs.Find(AfterJob);
		if (Index == INDEX_NONE)
		{
			RenderGridJobs.Add(Job);
		}
		else
		{
			RenderGridJobs.Insert(Job, Index + 1);
		}
	}
}

FString URenderGrid::GenerateUniqueRandomJobId()
{
	TArray<uint8> ByteArray;

	{// adding timestamp bytes >>
		const int64 Value = FDateTime::UtcNow().GetTicks();
		bool Add = false;
		for (int32 i = 56; i >= 0; i -= 8)
		{
			if (!Add)
			{
				if ((Value >> i) <= 0)
				{
					// don't add a 0 byte to the start
					continue;
				}
				Add = true;
			}
			ByteArray.Add(Value >> i);
		}
	}// adding timestamp bytes <<

	{// adding random bytes >>
		for (int32 i = 1; i <= 16; i++)
		{
			ByteArray.Add(FMath::Rand() & 0xff);
		}
	}// adding random bytes <<

	return FBase64::Encode(ByteArray).Replace(TEXT("="), TEXT("")).Replace(TEXT("/"), TEXT("_")).Replace(TEXT("+"), TEXT("-"));
}

FString URenderGrid::GenerateNextJobId()
{
	int32 Max = 0;
	for (URenderGridJob* Job : RenderGridJobs)
	{
		if (!IsValid(Job))
		{
			continue;
		}
		int32 Value = FCString::Atoi(*Job->GetJobId());
		if (Value > Max)
		{
			Max = Value;
		}
	}
	FString Result = FString::FromInt(Max + 1);
	while (Result.Len() < UE::RenderGrid::FRenderGridManager::GeneratedIdCharacterLength)
	{
		Result = TEXT("0") + Result;
	}
	return Result;
}

bool URenderGrid::DoesJobIdExist(const FString& JobId)
{
	const FString JobIdToLower = JobId.ToLower();
	for (URenderGridJob* Job : RenderGridJobs)
	{
		if (!IsValid(Job))
		{
			continue;
		}
		if (JobIdToLower == Job->GetJobId().ToLower())
		{
			return true;
		}
	}
	return false;
}

URenderGridJob* URenderGrid::CreateTempRenderGridJob()
{
	URenderGridJob* Job = NewObject<URenderGridJob>(this);
	Job->SetJobId(GenerateUniqueRandomJobId());
	Job->SetJobName(TEXT("New"));
	Job->SetLevelSequence(Defaults->LevelSequence);
	Job->SetRenderPreset(Defaults->RenderPreset);
	Job->SetOutputDirectoryRaw(Defaults->OutputDirectory);

	if (URenderGridPropsSourceRemoteControl* PropsSource = GetPropsSource<URenderGridPropsSourceRemoteControl>())
	{
		TArray<uint8> Bytes;
		for (URenderGridPropRemoteControl* Field : PropsSource->GetProps()->GetAllCasted())
		{
			if (Field->GetValue(Bytes))
			{
				if (const TSharedPtr<FRemoteControlEntity> FieldEntity = Field->GetRemoteControlEntity(); FieldEntity.IsValid())
				{
					Job->SetRemoteControlValueBytes(FieldEntity, Bytes);
				}
			}
		}
	}
	return Job;
}

URenderGridJob* URenderGrid::CreateAndAddNewRenderGridJob()
{
	URenderGridJob* Job = CreateTempRenderGridJob();
	Job->SetJobId(GenerateNextJobId());
	AddRenderGridJob(Job);
	return Job;
}

URenderGridJob* URenderGrid::DuplicateAndAddRenderGridJob(URenderGridJob* Job)
{
	if (!IsValid(Job))
	{
		return nullptr;
	}

	if (URenderGridJob* DuplicateRenderGridJob = DuplicateObject(Job, this); IsValid(DuplicateRenderGridJob))
	{
		DuplicateRenderGridJob->GenerateNewGuid();
		DuplicateRenderGridJob->SetJobId(GenerateNextJobId());
		InsertRenderGridJobAfter(DuplicateRenderGridJob, Job);
		return DuplicateRenderGridJob;
	}
	return nullptr;
}

bool URenderGrid::ReorderRenderGridJob(URenderGridJob* Job, URenderGridJob* DroppedOnJob, const bool bAfter)
{
	if (!IsValid(Job) || !IsValid(DroppedOnJob) || !HasRenderGridJob(Job) || !HasRenderGridJob(DroppedOnJob))
	{
		return false;
	}

	RemoveRenderGridJob(Job);
	if (bAfter)
	{
		InsertRenderGridJobAfter(Job, DroppedOnJob);
	}
	else
	{
		InsertRenderGridJobBefore(Job, DroppedOnJob);
	}
	return true;
}
