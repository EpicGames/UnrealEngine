// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGrid/RenderGridBlueprintGeneratedClass.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RenderGridBlueprintGeneratedClass)


uint8* URenderGridBlueprintGeneratedClass::GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const
{
	if (!IsInGameThread())
	{
		// we cant use the persistent frame if we are executing in parallel (as we could potentially thunk to BP)
		return nullptr;
	}
	return Super::GetPersistentUberGraphFrame(Obj, FuncToCheck);
}
