// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGrid/RenderGridManager.h"
#include "RenderGrid/RenderGridQueue.h"
#include "Utils/RenderGridUtils.h"
#include "MoviePipelineHighResSetting.h"


URenderGridQueue* UE::RenderGrid::FRenderGridManager::CreateBatchRenderQueue(URenderGrid* Grid, const TArray<URenderGridJob*>& Jobs)
{
	FRenderGridQueueCreateArgs JobArgs;
	JobArgs.RenderGrid = TStrongObjectPtr(Grid);
	JobArgs.RenderGridJobs.Append(Jobs);
	JobArgs.bIsBatchRender = true;
	URenderGridQueue* NewRenderQueue = URenderGridQueue::Create(JobArgs);
	if (!IsValid(NewRenderQueue))
	{
		return nullptr;
	}
	return NewRenderQueue;
}

URenderGridQueue* UE::RenderGrid::FRenderGridManager::CreateBatchRenderQueueSingleFrame(URenderGrid* Grid, const TArray<URenderGridJob*>& Jobs, const int32 Frame)
{
	FRenderGridQueueCreateArgs JobArgs;
	JobArgs.RenderGrid = TStrongObjectPtr(Grid);
	JobArgs.RenderGridJobs.Append(Jobs);
	JobArgs.bIsBatchRender = true;
	JobArgs.bForceUseSequenceFrameRate = true;
	JobArgs.Frame = Frame;
	URenderGridQueue* NewRenderQueue = URenderGridQueue::Create(JobArgs);
	if (!IsValid(NewRenderQueue))
	{
		return nullptr;
	}
	return NewRenderQueue;
}

URenderGridQueue* UE::RenderGrid::FRenderGridManager::CreateBatchRenderQueueSingleFramePosition(URenderGrid* Grid, const TArray<URenderGridJob*>& Jobs, const double FramePosition)
{
	FRenderGridQueueCreateArgs JobArgs;
	JobArgs.RenderGrid = TStrongObjectPtr(Grid);
	JobArgs.RenderGridJobs.Append(Jobs);
	JobArgs.bIsBatchRender = true;
	JobArgs.bForceUseSequenceFrameRate = true;
	JobArgs.FramePosition = FramePosition;
	URenderGridQueue* NewRenderQueue = URenderGridQueue::Create(JobArgs);
	if (!IsValid(NewRenderQueue))
	{
		return nullptr;
	}
	return NewRenderQueue;
}


URenderGridQueue* UE::RenderGrid::FRenderGridManager::RenderPreviewFrame(const FRenderGridManagerRenderPreviewFrameArgs& Args)
{
	const FRenderGridManagerRenderPreviewFrameArgsCallback Callback = Args.Callback;

	URenderGrid* RenderGrid = Args.RenderGrid.Get();
	if (!IsValid(RenderGrid) || RenderGrid->GetLevel().IsNull() || !RenderGrid->GetLevel().IsValid())
	{
		Callback.ExecuteIfBound(false);
		return nullptr;
	}

	URenderGridJob* RenderGridJob = Args.RenderGridJob.Get();
	if (!IsValid(RenderGridJob))
	{
		Callback.ExecuteIfBound(false);
		return nullptr;
	}

	URenderGridJob* JobCopy = DuplicateObject(RenderGridJob, RenderGridJob->GetOuter());
	if (!IsValid(JobCopy))
	{
		Callback.ExecuteIfBound(false);
		return nullptr;
	}

	JobCopy->SetJobId(JobCopy->GetGuid().ToString(EGuidFormats::Base36Encoded));

	if (Args.Frame.IsSet())
	{
		constexpr int32 RenderFramesCount = 1; // can be more than 1 to prevent rendering issues, will always take the last frame that's rendered

		JobCopy->SetIsUsingCustomStartFrame(true);
		JobCopy->SetCustomStartFrame(Args.Frame.Get(0));

		JobCopy->SetIsUsingCustomEndFrame(true);
		JobCopy->SetCustomEndFrame(Args.Frame.Get(0));

		if (!JobCopy->SetSequenceEndFrame(JobCopy->GetSequenceStartFrame().Get(0) + 1) ||
			!JobCopy->SetSequenceStartFrame(JobCopy->GetSequenceEndFrame().Get(0) - RenderFramesCount))
		{
			Callback.ExecuteIfBound(false);
			return nullptr;
		}
	}

	JobCopy->SetIsUsingCustomResolution(true);
	JobCopy->SetCustomResolution(Args.Resolution);

	JobCopy->SetOutputDirectory(TmpRenderedFramesPath / (Args.Frame.IsSet() ? TEXT("PreviewFrame") : TEXT("PreviewFrames")));

	FRenderGridQueueCreateArgs JobArgs;
	JobArgs.RenderGrid = TStrongObjectPtr(RenderGrid);
	JobArgs.RenderGridJobs.Add(TStrongObjectPtr(JobCopy));
	JobArgs.bHeadless = Args.bHeadless;
	JobArgs.bForceOutputImage = true;
	JobArgs.bForceOnlySingleOutput = true;
	JobArgs.bForceUseSequenceFrameRate = Args.Frame.IsSet();
	JobArgs.bEnsureSequentialFilenames = true;
	JobArgs.DisablePipelineSettingsClasses.Add(UMoviePipelineAntiAliasingSetting::StaticClass());
	JobArgs.DisablePipelineSettingsClasses.Add(UMoviePipelineHighResSetting::StaticClass());

	URenderGridQueue* NewRenderQueue = URenderGridQueue::Create(JobArgs);
	if (!NewRenderQueue)
	{
		Callback.ExecuteIfBound(false);
		return nullptr;
	}

	const FGuid JobId = JobCopy->GetGuid();
	const TOptional<int32> StartFrameOfRender = (Args.Frame.IsSet() ? TOptional<int32>() : JobCopy->GetStartFrame());
	NewRenderQueue->OnExecuteFinished().AddLambda([this, Callback, JobId, StartFrameOfRender](URenderGridQueue* Queue, const bool bSuccess)
	{
		if (StartFrameOfRender.IsSet())
		{
			StartFrameOfRenders.Add(JobId, StartFrameOfRender.Get(0));
		}
		Callback.ExecuteIfBound(bSuccess);
	});

	NewRenderQueue->Execute();
	return NewRenderQueue;
}

UTexture2D* UE::RenderGrid::FRenderGridManager::GetSingleRenderedPreviewFrame(URenderGridJob* Job, UTexture2D* ReusingTexture2D, bool& bOutReusedGivenTexture2D)
{
	static const FString PreviewFramesDir = TmpRenderedFramesPath / TEXT("PreviewFrame");
	if (!IsValid(Job))
	{
		bOutReusedGivenTexture2D = (ReusingTexture2D == nullptr);
		return nullptr;
	}
	const FString PreviewFramesSubDir = Job->GetGuid().ToString(EGuidFormats::Base36Encoded);

	TArray<FString> ImagePaths = Private::FRenderGridUtils::GetFiles(PreviewFramesDir / PreviewFramesSubDir, true);
	ImagePaths.Sort();
	Algo::Reverse(ImagePaths);
	for (const FString& ImagePath : ImagePaths)
	{
		if (UTexture2D* Texture = Private::FRenderGridUtils::GetImage(ImagePath, ReusingTexture2D, bOutReusedGivenTexture2D))
		{
			return Texture;
		}
	}
	bOutReusedGivenTexture2D = (ReusingTexture2D == nullptr);
	return nullptr;
}

UTexture2D* UE::RenderGrid::FRenderGridManager::GetSingleRenderedPreviewFrame(URenderGridJob* Job)
{
	bool bOutReusedGivenTexture2D;
	return GetSingleRenderedPreviewFrame(Job, nullptr, bOutReusedGivenTexture2D);
}

UTexture2D* UE::RenderGrid::FRenderGridManager::GetRenderedPreviewFrame(URenderGridJob* Job, const int32 Frame, UTexture2D* ReusingTexture2D, bool& bOutReusedGivenTexture2D)
{
	static const FString PreviewFramesDir = TmpRenderedFramesPath / TEXT("PreviewFrames");
	if (!IsValid(Job))
	{
		bOutReusedGivenTexture2D = (ReusingTexture2D == nullptr);
		return nullptr;
	}
	const FString PreviewFramesSubDir = Job->GetGuid().ToString(EGuidFormats::Base36Encoded);

	int32 CurrentFrame = 0;
	if (int32* StartFrameOfRenderPtr = StartFrameOfRenders.Find(Job->GetGuid()))
	{
		CurrentFrame = *StartFrameOfRenderPtr;
	}

	TArray<FString> ImagePaths = Private::FRenderGridUtils::GetFiles(PreviewFramesDir / PreviewFramesSubDir, true);
	for (const FString& ImagePath : ImagePaths)
	{
		if (!Private::FRenderGridUtils::IsImage(ImagePath))
		{
			continue;
		}
		if (CurrentFrame == Frame)
		{
			return Private::FRenderGridUtils::GetImage(ImagePath, ReusingTexture2D, bOutReusedGivenTexture2D);
		}
		CurrentFrame++;
	}

	bOutReusedGivenTexture2D = (ReusingTexture2D == nullptr);
	return nullptr;
}

UTexture2D* UE::RenderGrid::FRenderGridManager::GetRenderedPreviewFrame(URenderGridJob* Job, const int32 Frame)
{
	bool bOutReusedGivenTexture2D;
	return GetRenderedPreviewFrame(Job, Frame, nullptr, bOutReusedGivenTexture2D);
}


void UE::RenderGrid::FRenderGridManager::UpdateStoredRenderGridJobsPropValues(URenderGrid* Grid)
{
	if (!IsValid(Grid))
	{
		return;
	}

	URenderGridPropsSourceRemoteControl* PropsSource = Grid->GetPropsSource<URenderGridPropsSourceRemoteControl>();
	if (!IsValid(PropsSource))
	{
		return;
	}

	TArray<URenderGridJob*> Jobs = Grid->GetRenderGridJobs();
	TArray<uint8> Bytes;
	for (URenderGridPropRemoteControl* Field : PropsSource->GetProps()->GetAllCasted())
	{
		if (!Field->GetValue(Bytes))
		{
			continue;
		}
		if (const TSharedPtr<FRemoteControlEntity> Entity = Field->GetRemoteControlEntity(); Entity.IsValid())
		{
			for (URenderGridJob* Job : Jobs)
			{
				if (!Job->HasStoredRemoteControlValueBytes(Entity))
				{
					Job->SetRemoteControlValueBytes(Entity, Bytes);
				}
			}
		}
	}
}

FRenderGridManagerPreviousPropValues UE::RenderGrid::FRenderGridManager::ApplyJobPropValues(const URenderGrid* Grid, const URenderGridJob* Job)
{
	FRenderGridManagerPreviousPropValues PreviousPropValues;

	if (!IsValid(Grid) || !IsValid(Job))
	{
		return PreviousPropValues;
	}

	URenderGridPropsSourceBase* PropsSource = Grid->GetPropsSource();
	if (URenderGridPropsSourceRemoteControl* PropsSourceRC = Cast<URenderGridPropsSourceRemoteControl>(PropsSource))
	{
		for (URenderGridPropRemoteControl* Prop : PropsSourceRC->GetProps()->GetAllCasted())
		{
			TArray<uint8> PreviousPropData;
			if (!Prop->GetValue(PreviousPropData))
			{
				continue;
			}

			TArray<uint8> PropData;
			if (!Job->GetRemoteControlValueBytes(Prop->GetRemoteControlEntity(), PropData))
			{
				continue;
			}

			PreviousPropValues.RemoteControlData.Add(Prop, PreviousPropData);
			Prop->SetValue(PropData);
		}
		return PreviousPropValues;
	}
	return PreviousPropValues;
}

void UE::RenderGrid::FRenderGridManager::RestorePropValues(const FRenderGridManagerPreviousPropValues& PreviousPropValues)
{
	for (const auto& PropValuePair : PreviousPropValues.RemoteControlData)
	{
		if (IsValid(PropValuePair.Key))
		{
			PropValuePair.Key->SetValue(PropValuePair.Value.Bytes);
		}
	}
}
