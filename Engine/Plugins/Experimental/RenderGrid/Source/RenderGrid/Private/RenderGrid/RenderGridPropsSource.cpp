// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGrid/RenderGridPropsSource.h"
#include "RenderGridLog.h"
#include "IRemoteControlModule.h"
#include "RCVirtualProperty.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"


bool /*URenderGridPropRemoteControl::*/GetObjectRef(const TSharedPtr<FRemoteControlProperty>& Field, const ERCAccess Access, FRCObjectReference& OutObjectRef)
{
	if (!Field.IsValid())
	{
		return false;
	}

	if (const UScriptStruct* PropertyStruct = Field->GetStruct())
	{
		if (PropertyStruct->IsChildOf(FRemoteControlProperty::StaticStruct()))
		{
			if (UObject* FieldBoundObject = Field->GetBoundObject(); IsValid(FieldBoundObject))
			{
				FRCObjectReference ObjectRef;
				FString* ErrorText = nullptr;

				if (IRemoteControlModule::Get().ResolveObjectProperty(Access, FieldBoundObject, Field->FieldPathInfo, ObjectRef, ErrorText))
				{
					OutObjectRef = ObjectRef;
					return true;
				}

				UE_LOGF(LogRenderGrid, Warning, "Couldn\'t resolve object property \"%ls\" in object \"%ls\": %ls", *Field->FieldName.ToString(), *FieldBoundObject->GetPathName(), (ErrorText ? **ErrorText : TEXT("unknown")));
			}
		}
	}

	return false;
}

bool URenderGridPropRemoteControl::GetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, TArray<uint8>& OutBytes)
{
	OutBytes.Empty();

	TSharedPtr<FRemoteControlProperty> Field = StaticCastSharedPtr<FRemoteControlProperty>(RemoteControlEntity);
	if (!Field.IsValid())
	{
		return false;
	}

	FRCObjectReference ObjectRef;
	if (!GetObjectRef(Field, ERCAccess::READ_ACCESS, ObjectRef))
	{
		return false;
	}
	FMemoryWriter Writer = FMemoryWriter(OutBytes);
	FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
	return IRemoteControlModule::Get().GetObjectProperties(ObjectRef, WriterBackend);
}

bool URenderGridPropRemoteControl::SetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes)
{
	TSharedPtr<FRemoteControlProperty> Field = StaticCastSharedPtr<FRemoteControlProperty>(RemoteControlEntity);
	if (!Field.IsValid())
	{
		return false;
	}

	FRCObjectReference ObjectRefRead;
	if (GetObjectRef(Field, ERCAccess::READ_ACCESS, ObjectRefRead))
	{
		TArray<uint8> CurrentBytes;
		FMemoryWriter Writer = FMemoryWriter(CurrentBytes);
		FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		if (IRemoteControlModule::Get().GetObjectProperties(ObjectRefRead, WriterBackend))
		{
			if (CurrentBytes == Bytes)
			{
				// if the given value is already set, don't do anything
				return true;
			}
		}
	}

	FRCObjectReference ObjectRefWrite;
	if (!GetObjectRef(Field, ERCAccess::WRITE_ACCESS, ObjectRefWrite))
	{
		return false;
	}
	FMemoryReader Reader = FMemoryReader(Bytes);
	FJsonStructDeserializerBackend ReaderBackend = FJsonStructDeserializerBackend(Reader);
	if (!IRemoteControlModule::Get().SetObjectProperties(ObjectRefWrite, ReaderBackend, ERCPayloadType::Json))
	{
		return false;
	}

	{
		/**
		 * Work-around.
		 * 
		 * Remote control doesn't fire the PreEditChange and PostEditChangeProperty functions right away,
		 * causing construction scripts to not fire right away,
		 * which causes changes made in construction scripts to not be rendered when using the movie pipeline plugin.
		 * 
		 * The work-around is to call the functions manually.
		 */

		if (UObject* Object = ObjectRefWrite.Object.Get(); IsValid(Object))
		{
			FEditPropertyChain PreEditChain;
			ObjectRefWrite.PropertyPathInfo.ToEditPropertyChain(PreEditChain);
			Object->PreEditChange(PreEditChain);

			FPropertyChangedEvent PropertyEvent = ObjectRefWrite.PropertyPathInfo.ToPropertyChangedEvent();
			PropertyEvent.ChangeType = EPropertyChangeType::ValueSet;
			Object->PostEditChangeProperty(PropertyEvent);
		}
	}

	return true;
}

bool URenderGridPropRemoteControl::CanSetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes)
{
	TSharedPtr<FRemoteControlProperty> Field = StaticCastSharedPtr<FRemoteControlProperty>(RemoteControlEntity);
	if (!Field.IsValid())
	{
		return false;
	}

	FRCObjectReference ObjectRefRead;
	if (GetObjectRef(Field, ERCAccess::READ_ACCESS, ObjectRefRead))
	{
		TArray<uint8> CurrentBytes;
		FMemoryWriter Writer = FMemoryWriter(CurrentBytes);
		FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		if (IRemoteControlModule::Get().GetObjectProperties(ObjectRefRead, WriterBackend))
		{
			if (CurrentBytes == Bytes)
			{
				// if the given value is already set, don't do anything
				return true;
			}
		}
	}

	FRCObjectReference ObjectRefWrite;
	if (!GetObjectRef(Field, ERCAccess::WRITE_ACCESS, ObjectRefWrite))
	{
		return false;
	}
	return true;
}

void URenderGridPropRemoteControl::Initialize(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity)
{
	RemoteControlEntity = InRemoteControlEntity;
}


void URenderGridPropsRemoteControl::Initialize(URemoteControlPreset* InRemoteControlPreset)
{
	RemoteControlPreset = InRemoteControlPreset;
}

TArray<URenderGridPropBase*> URenderGridPropsRemoteControl::GetAll() const
{
	TArray<URenderGridPropBase*> Result;
	if (IsValid(RemoteControlPreset))
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				URenderGridPropRemoteControl* PropObj = NewObject<URenderGridPropRemoteControl>(const_cast<URenderGridPropsRemoteControl*>(this));
				PropObj->Initialize(Prop);
				Result.Add(PropObj);
			}
		}
	}
	return Result;
}

TArray<URenderGridPropRemoteControl*> URenderGridPropsRemoteControl::GetAllCasted() const
{
	TArray<URenderGridPropRemoteControl*> Result;
	if (IsValid(RemoteControlPreset))
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				URenderGridPropRemoteControl* PropObj = NewObject<URenderGridPropRemoteControl>(const_cast<URenderGridPropsRemoteControl*>(this));
				PropObj->Initialize(Prop);
				Result.Add(PropObj);
			}
		}
	}
	return Result;
}

URenderGridPropRemoteControl* URenderGridPropsRemoteControl::GetById(const FGuid& Id) const
{
	if (IsValid(RemoteControlPreset))
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				if (Prop->GetId() == Id)
				{
					URenderGridPropRemoteControl* PropObj = NewObject<URenderGridPropRemoteControl>(const_cast<URenderGridPropsRemoteControl*>(this));
					PropObj->Initialize(Prop);
					return PropObj;
				}
			}
		}
	}
	return nullptr;
}

URenderGridPropRemoteControl* URenderGridPropsRemoteControl::GetByLabel(const FString& Label) const
{
	if (IsValid(RemoteControlPreset))
	{
		for (const TWeakPtr<FRemoteControlEntity>& PropWeakPtr : RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (const TSharedPtr<FRemoteControlEntity> Prop = PropWeakPtr.Pin())
			{
				if (Prop->GetLabel().ToString().TrimStartAndEnd().Equals(Label.TrimStartAndEnd(), ESearchCase::CaseSensitive))
				{
					URenderGridPropRemoteControl* PropObj = NewObject<URenderGridPropRemoteControl>(const_cast<URenderGridPropsRemoteControl*>(this));
					PropObj->Initialize(Prop);
					return PropObj;
				}
			}
		}
	}
	return nullptr;
}


void URenderGridPropsSourceRemoteControl::SetSourceOrigin(UObject* SourceOrigin)
{
	if (!IsValid(SourceOrigin))
	{
		RemoteControlPreset = nullptr;
		return;
	}

	RemoteControlPreset = Cast<URemoteControlPreset>(SourceOrigin);
	if (IsValid(RemoteControlPreset))
	{
		ActivePresetGroup = RemoteControlPreset->Layout.GetDefaultGroup().Name;
	}
}

URenderGridPropsRemoteControl* URenderGridPropsSourceRemoteControl::GetProps() const
{
	URenderGridPropsRemoteControl* PropsObj = NewObject<URenderGridPropsRemoteControl>(const_cast<URenderGridPropsSourceRemoteControl*>(this));
	PropsObj->Initialize(IsValid(RemoteControlPreset) ? RemoteControlPreset : nullptr);
	return PropsObj;
}

void URenderGridPropsSourceRemoteControl::GetAvailablePresetGroups(TArray<FName>& OutPresetGroups) const
{
	OutPresetGroups.Reset();

	if (IsValid(RemoteControlPreset))
	{
		TArray<FRemoteControlPresetGroup> PresetGroups = RemoteControlPreset->Layout.GetGroups();
		for (const FRemoteControlPresetGroup& PresetGroup : PresetGroups)
		{
			OutPresetGroups.Add(PresetGroup.Name);
		}
	}
}
