// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGrid/RenderGridQueue.h"

#include "IRenderGridModule.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridManager.h"
#include "Utils/RenderGridUtils.h"
#include "UObject/Package.h"
#include "Utils/RenderGridGenericExecutionQueue.h"

#include "Dom/JsonObject.h"
#include "FileHelpers.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorModule.h"
#include "Modules/ModuleManager.h"
#include "MoviePipelineAntiAliasingSetting.h"
#include "MoviePipelineEditorBlueprintLibrary.h"
#include "MoviePipelineExecutor.h"
#include "MoviePipelineImageSequenceOutput.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePIEExecutor.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineQueue.h"


URenderGridMoviePipelineRenderJob* URenderGridMoviePipelineRenderJob::Create(URenderGridQueue* Queue, URenderGridJob* Job, const UE::RenderGrid::FRenderGridQueueCreateArgs& Args)
{
	if (!IsValid(Queue) || !IsValid(Job))
	{
		return nullptr;
	}

	URenderGrid* RenderGrid = Args.RenderGrid.Get();
	if (!IsValid(RenderGrid))
	{
		return nullptr;
	}

	const UClass* PipelineExecutor = (IsValid(*Args.PipelineExecutorClass) ? *Args.PipelineExecutorClass : UMoviePipelinePIEExecutor::StaticClass());
	if (!PipelineExecutor)
	{
		return nullptr;
	}

	URenderGridMoviePipelineRenderJob* RenderJob = NewObject<URenderGridMoviePipelineRenderJob>(Queue);
	RenderJob->RenderGridJob = Job;
	RenderJob->RenderGrid = RenderGrid;
	RenderJob->PipelineQueue = NewObject<UMoviePipelineQueue>(RenderJob);
	RenderJob->PipelineExecutor = NewObject<UMoviePipelineExecutorBase>(RenderJob, PipelineExecutor);
	RenderJob->PipelineExecutorJob = nullptr;
	RenderJob->Status = TEXT("Skipped");
	RenderJob->bCanExecute = false;
	RenderJob->bCanceled = false;
	RenderJob->bFinished = false;
	RenderJob->Promise = MakeShared<TPromise<void>>();
	RenderJob->Promise->SetValue();
	RenderJob->PromiseFuture = RenderJob->Promise->GetFuture().Share();
	RenderJob->Promise.Reset();

	if (Args.bHeadless)
	{
		if (UMoviePipelinePIEExecutor* ActiveExecutorPIE = Cast<UMoviePipelinePIEExecutor>(RenderJob->PipelineExecutor))
		{
			ActiveExecutorPIE->SetIsRenderingOffscreen(true);
		}
	}


	ULevelSequence* JobSequence = Job->GetLevelSequence();
	if (!IsValid(JobSequence) || !Job->GetSequenceStartFrame().IsSet() || !Job->GetSequenceEndFrame().IsSet() || (Job->GetSequenceStartFrame().Get(0) >= Job->GetSequenceEndFrame().Get(0)))
	{
		return RenderJob;
	}

	UMoviePipelineExecutorJob* NewJob = UMoviePipelineEditorBlueprintLibrary::CreateJobFromSequence(RenderJob->PipelineQueue, JobSequence);
	RenderJob->PipelineExecutorJob = NewJob;

	FSoftObjectPath Level = RenderGrid->GetLevel().ToSoftObjectPath();
	if (Level.IsValid())
	{
		NewJob->Map = Level;
	}

	UMoviePipelinePrimaryConfig* JobRenderPreset = Job->GetRenderPreset();
	if (IsValid(JobRenderPreset))
	{
		NewJob->SetConfiguration(JobRenderPreset);
	}
	else
	{
		UMoviePipelineEditorBlueprintLibrary::EnsureJobHasDefaultSettings(NewJob);
	}

	if (Args.DisablePipelineSettingsClasses.Num() > 0)
	{
		for (UMoviePipelineSetting* Setting : NewJob->GetConfiguration()->FindSettings<UMoviePipelineSetting>())
		{
			if (!IsValid(Setting))
			{
				continue;
			}
			for (TSubclassOf<UMoviePipelineSetting> DisableSettingsClass : Args.DisablePipelineSettingsClasses)
			{
				if (Setting->IsA(DisableSettingsClass))
				{
					Setting->SetIsEnabled(false);
					break;
				}
			}
		}
	}

	if (Args.bForceOutputImage || Args.bForceOnlySingleOutput)
	{
		bool bFound = false;
		const bool bContainsPreferredType = IsValid(NewJob->GetConfiguration()->FindSetting<UMoviePipelineImageSequenceOutput_PNG>());
		for (UMoviePipelineOutputBase* Setting : NewJob->GetConfiguration()->FindSettings<UMoviePipelineOutputBase>())
		{
			if (!IsValid(Setting))
			{
				continue;
			}
			if (Cast<UMoviePipelineImageSequenceOutput_PNG>(Setting) || Cast<UMoviePipelineImageSequenceOutput_JPG>(Setting) || Cast<UMoviePipelineImageSequenceOutput_BMP>(Setting))
			{
				if (!Args.bForceOnlySingleOutput || (!bFound && (!bContainsPreferredType || Cast<UMoviePipelineImageSequenceOutput_PNG>(Setting))))
				{
					bFound = true;
					continue;
				}
			}
			Setting->SetIsEnabled(false);
		}
		if (Args.bForceOutputImage && !bFound)
		{
			if (UMoviePipelineImageSequenceOutput_PNG* NewSetting = Cast<UMoviePipelineImageSequenceOutput_PNG>(NewJob->GetConfiguration()->FindOrAddSettingByClass(UMoviePipelineImageSequenceOutput_PNG::StaticClass())))
			{
				NewSetting->bWriteAlpha = false;
			}
		}
	}

	if (UMoviePipelineAntiAliasingSetting* ExistingAntiAliasingSettings = NewJob->GetConfiguration()->FindSetting<UMoviePipelineAntiAliasingSetting>())
	{
		// anti-aliasing settings already present (and enabled)
		if ((UE::MovieRenderPipeline::GetEffectiveAntiAliasingMethod(ExistingAntiAliasingSettings) == AAM_FXAA) && (ExistingAntiAliasingSettings->SpatialSampleCount <= 1) && (ExistingAntiAliasingSettings->TemporalSampleCount <= 1))
		{
			ExistingAntiAliasingSettings->TemporalSampleCount = 2; // FXAA transparency fix
		}
	}
	else if (UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = Cast<UMoviePipelineAntiAliasingSetting>(NewJob->GetConfiguration()->FindOrAddSettingByClass(UMoviePipelineAntiAliasingSetting::StaticClass())))
	{
		// anti-aliasing settings not yet present (or enabled), created a new one
		AntiAliasingSettings->EngineWarmUpCount = 0;
		AntiAliasingSettings->RenderWarmUpCount = 0;
		AntiAliasingSettings->SpatialSampleCount = 1;
		AntiAliasingSettings->TemporalSampleCount = 2;
		AntiAliasingSettings->bOverrideAntiAliasing = true;
		AntiAliasingSettings->AntiAliasingMethod = AAM_FXAA;
	}

	bool bHasShot = false;
	for (const UMoviePipelineExecutorShot* Shot : NewJob->ShotInfo)
	{
		if (!Shot)
		{
			continue;
		}
		bHasShot = true;

		UMoviePipelineOutputSetting* Setting = Cast<UMoviePipelineOutputSetting>(UMoviePipelineBlueprintLibrary::FindOrGetDefaultSettingForShot(UMoviePipelineOutputSetting::StaticClass(), NewJob->GetConfiguration(), Shot));

		Setting->bUseCustomPlaybackRange = true;
		Setting->CustomStartFrame = Job->GetSequenceStartFrame().Get(0);
		Setting->CustomEndFrame = Job->GetSequenceEndFrame().Get(0);

		if (Args.Frame.IsSet())
		{
			Setting->CustomStartFrame = Args.Frame.Get(0);
			Setting->CustomEndFrame = Setting->CustomStartFrame + 1;
		}
		else if (Args.FramePosition.IsSet())
		{
			double FramePosition = FMath::Clamp<double>(Args.FramePosition.Get(0.0), 0.0, 1.0);
			int32 Frame = FMath::Lerp<int32, double>(Setting->CustomStartFrame, Setting->CustomEndFrame - 1, FramePosition);
			Setting->CustomStartFrame = Frame;
			Setting->CustomEndFrame = Frame + 1;
		}

		if (Args.bForceUseSequenceFrameRate)
		{
			Setting->bUseCustomFrameRate = false;
		}

		if (Job->GetIsUsingCustomResolution())
		{
			Setting->OutputResolution = Job->GetCustomResolution();
		}

		const FString JobOutputRootDirectory = Job->GetOutputDirectory();
		const FString JobId = Job->GetJobId();
		if (!JobOutputRootDirectory.IsEmpty() && !JobId.IsEmpty())
		{
			const FString JobOutputDirectory = JobOutputRootDirectory / JobId;
			UE::RenderGrid::Private::FRenderGridUtils::DeleteDirectory(JobOutputDirectory);
			Setting->OutputDirectory.Path = JobOutputDirectory;
		}

		if (Args.bEnsureSequentialFilenames || !IsValid(JobRenderPreset))
		{
			Setting->FileNameFormat = TEXT("{frame_number}");
			Setting->ZeroPadFrameNumbers = 10;
			Setting->FrameNumberOffset = 1000000000;
		}
	}
	if (!bHasShot)
	{
		return RenderJob;
	}

	RenderJob->Status = TEXT("Waiting");
	RenderJob->bCanExecute = true;
	return RenderJob;
}

void URenderGridMoviePipelineRenderJob::BeginDestroy()
{
	if (Promise.IsValid())
	{
		Promise->SetValue();
		Promise.Reset();
	}
	Super::BeginDestroy();
}

TSharedFuture<void> URenderGridMoviePipelineRenderJob::Execute()
{
	if (PipelineExecutor->IsRendering())
	{
		return PromiseFuture;
	}

	Promise = MakeShared<TPromise<void>>();
	PromiseFuture = Promise->GetFuture().Share();

	if (!bCanExecute)
	{
		Status = TEXT("Skipped");
		Promise->SetValue();
		Promise.Reset();
		return PromiseFuture;
	}
	if (bCanceled)
	{
		Status = TEXT("Canceled");
		Promise->SetValue();
		Promise.Reset();
		return PromiseFuture;
	}

	AddToRoot();
	if (ILevelSequenceEditorModule* LevelSequenceEditorModule = FModuleManager::GetModulePtr<ILevelSequenceEditorModule>("LevelSequenceEditor"); LevelSequenceEditorModule)
	{
		LevelSequenceEditorModule->OnComputePlaybackContext().AddUObject(this, &URenderGridMoviePipelineRenderJob::ComputePlaybackContext);
	}
	Status = TEXT("Rendering...");
	PipelineExecutor->OnExecutorFinished().AddUObject(this, &URenderGridMoviePipelineRenderJob::ExecuteFinished);
	PipelineExecutor->Execute(PipelineQueue);
	return PromiseFuture;
}

void URenderGridMoviePipelineRenderJob::Cancel()
{
	bCanceled = true;
	bFinished = true;
	if (PipelineExecutor->IsRendering())
	{
		PipelineExecutor->CancelAllJobs();
	}
}

FString URenderGridMoviePipelineRenderJob::GetStatus() const
{
	if (UMoviePipelineExecutorJob* RenderExecutorJob = PipelineExecutorJob.Get(); IsValid(RenderExecutorJob))
	{
		if (FString JobStatus = RenderExecutorJob->GetStatusMessage().TrimStartAndEnd(); !JobStatus.IsEmpty())
		{
			return JobStatus;
		}
	}
	return Status;
}

float URenderGridMoviePipelineRenderJob::GetStatusPercentage() const
{
	if (bCanceled || bFinished || !bCanExecute)
	{
		return 100;
	}

	if (UMoviePipelineExecutorJob* RenderExecutorJob = PipelineExecutorJob.Get(); IsValid(RenderExecutorJob))
	{
		return FMath::Clamp(RenderExecutorJob->GetStatusProgress() * 100, 0, 100);
	}
	return 0;
}

int32 URenderGridMoviePipelineRenderJob::GetEngineWarmUpCount() const
{
	if (UMoviePipelineExecutorJob* RenderExecutorJob = PipelineExecutorJob.Get(); IsValid(RenderExecutorJob))
	{
		if (UMoviePipelineAntiAliasingSetting* AntiAliasingSettings = Cast<UMoviePipelineAntiAliasingSetting>(RenderExecutorJob->GetConfiguration()->FindOrAddSettingByClass(UMoviePipelineAntiAliasingSetting::StaticClass())))
		{
			return FMath::Max<int32>(0, AntiAliasingSettings->EngineWarmUpCount);
		}
	}
	return 0;
}

void URenderGridMoviePipelineRenderJob::OnWaitForEngineWarmUpCount()
{
	if (!bCanceled && !bFinished)
	{
		Status = TEXT("Warming up...");
	}
}

void URenderGridMoviePipelineRenderJob::OnPreRenderEvent()
{
	if (!bCanceled && !bFinished)
	{
		Status = TEXT("Starting...");
	}
}

void URenderGridMoviePipelineRenderJob::OnPostRenderEvent()
{
	// the "Done" status is already set by the ExecuteFinished callback at this point
}

void URenderGridMoviePipelineRenderJob::ComputePlaybackContext(bool& bOutAllowBinding)
{
	bOutAllowBinding = false;
}

void URenderGridMoviePipelineRenderJob::ExecuteFinished(UMoviePipelineExecutorBase* InPipelineExecutor, const bool bSuccess)
{
	if (ILevelSequenceEditorModule* LevelSequenceEditorModule = FModuleManager::GetModulePtr<ILevelSequenceEditorModule>("LevelSequenceEditorModule"))
	{
		LevelSequenceEditorModule->OnComputePlaybackContext().RemoveAll(this);
	}
	bCanceled = (bCanceled || !bSuccess);
	bFinished = true;
	Status = (bCanceled ? TEXT("Canceled") : TEXT("Done"));
	if (Promise.IsValid())
	{
		Promise->SetValue();
		Promise.Reset();
	}
	RemoveFromRoot();
}


TArray<TObjectPtr<URenderGridQueue>> URenderGridQueue::RenderingQueues;
URenderGridQueue::FOnRenderingQueueChanged URenderGridQueue::OnRenderingQueueChangedDelegate;
bool URenderGridQueue::bExitOnCompletion = false;

void URenderGridQueue::CloseEditorOnCompletion()
{
	bExitOnCompletion = true;
	RequestAppExitIfSetToExitOnCompletion();
}

void URenderGridQueue::RequestAppExitIfSetToExitOnCompletion()
{
	if (bExitOnCompletion && !IsRenderingAny())
	{
		FPlatformMisc::RequestExit(false);
	}
}

bool URenderGridQueue::IsRenderingAny()
{
	return (GetCurrentlyRenderingQueue() != nullptr);
}

URenderGridQueue* URenderGridQueue::GetCurrentlyRenderingQueue()
{
	bool bRemovedAny = false;
	while (!RenderingQueues.IsEmpty())
	{
		URenderGridQueue* Queue = RenderingQueues[0];
		if (!IsValid(Queue))
		{
			RenderingQueues.RemoveAt(0);
			bRemovedAny = true;
			continue;
		}
		if (bRemovedAny)
		{
			if (!Queue->bPaused)
			{
				Queue->Queue->Start();
			}
			OnRenderingQueueChanged().Broadcast();
		}
		return Queue;
	}
	return nullptr;
}

int32 URenderGridQueue::GetRemainingRenderingQueuesCount()
{
	return RenderingQueues.Num();
}

void URenderGridQueue::AddRenderingQueue(URenderGridQueue* Queue)
{
	if (!IsValid(Queue))
	{
		return;
	}
	RenderingQueues.Add(Queue);
	if ((GetCurrentlyRenderingQueue() == Queue) && !Queue->bPaused)
	{
		Queue->Queue->Start(); // can be called twice here, which is fine (once in GetCurrentlyExecutingQueue() if the previous Queue became invalid, and then once again here)
	}
	OnRenderingQueueChanged().Broadcast();
}

void URenderGridQueue::DoNextRenderingQueue()
{
	if (RenderingQueues.IsEmpty())
	{
		return;
	}
	RenderingQueues.RemoveAt(0);
	if (URenderGridQueue* Queue = GetCurrentlyRenderingQueue())
	{
		if (!Queue->bPaused)
		{
			Queue->Queue->Start(); // can be called twice here, which is fine (once in GetCurrentlyExecutingQueue() if the previous Queue became invalid, and then once again here)
		}
		OnRenderingQueueChanged().Broadcast();
	}
	else
	{
		OnRenderingQueueChanged().Broadcast();
		RequestAppExitIfSetToExitOnCompletion();
	}
}


void URenderGridQueue::Tick(float DeltaTime)
{
	if (bStarted && bFinished)
	{
		if (OnExecuteFinishedDelegate.IsBound())
		{
			OnExecuteFinishedDelegate.Broadcast(this, !bCanceled);
			OnExecuteFinishedDelegate.Clear();
		}
	}
}

URenderGridQueue* URenderGridQueue::Create(const UE::RenderGrid::FRenderGridQueueCreateArgs& Args)
{
	URenderGrid* RenderGrid = Args.RenderGrid.Get();
	if (!IsValid(RenderGrid) || RenderGrid->GetLevel().IsNull())
	{
		return nullptr;
	}

	URenderGridQueue* RenderQueue = NewObject<URenderGridQueue>(GetTransientPackage());
	RenderQueue->Guid = FGuid::NewGuid();
	RenderQueue->Args = Args;
	RenderQueue->Queue = MakeShareable(new UE::RenderGrid::Private::FRenderGridGenericExecutionQueue);
	RenderQueue->RenderGrid = RenderGrid;
	RenderQueue->bStarted = false;
	RenderQueue->bCanceled = false;
	RenderQueue->bFinished = false;

	for (const TStrongObjectPtr<URenderGridJob>& JobPtr : Args.RenderGridJobs)
	{
		if (URenderGridJob* Job = JobPtr.Get(); IsValid(Job))
		{
			RenderQueue->AddJob(Job);
		}
	}
	return RenderQueue;
}

FString URenderGridQueue::ToDebugString() const
{
	return UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(ToDebugJson(), true);
}

TSharedPtr<FJsonValue> URenderGridQueue::ToDebugJson() const
{
	TSharedPtr<FJsonObject> Root = MakeShareable(new FJsonObject());
	{
		Root->SetStringField(TEXT("Class"), GetParentNativeClass(GetClass())->GetName());
		Root->SetStringField(TEXT("Object"), GetName());
		Root->SetStringField(TEXT("Guid"), GetGuid().ToString());
		Root->SetBoolField(TEXT("IsStarted"), IsStarted());
		Root->SetBoolField(TEXT("IsPaused"), IsPaused());
		Root->SetBoolField(TEXT("IsCanceled"), IsCanceled());
		Root->SetBoolField(TEXT("IsFinished"), IsFinished());
		Root->SetBoolField(TEXT("IsCurrentlyRendering"), IsCurrentlyRendering());
		Root->SetStringField(TEXT("Status"), GetStatus());
		Root->SetNumberField(TEXT("StatusPercentage"), GetStatusPercentage());

		{
			TArray<TSharedPtr<FJsonValue>> Array;
			for (URenderGridJob* Job : GetJobs())
			{
				TSharedPtr<FJsonObject> Object = MakeShareable(new FJsonObject());
				Object->SetStringField(TEXT("Status"), GetJobStatus(Job));
				Object->SetNumberField(TEXT("StatusPercentage"), GetJobStatusPercentage(Job));
				Object->SetField(TEXT("Job"), Job->ToDebugJson());
				Array.Add(MakeShareable(new FJsonValueObject(Object)));
			}
			Root->SetArrayField(TEXT("Jobs"), Array);
		}
	}
	return MakeShareable(new FJsonValueObject(Root));
}

void URenderGridQueue::Execute()
{
	OnStart();
}


void URenderGridQueue::AddJob(URenderGridJob* Job)
{
	if (!IsValid(Job) || bCanceled || bFinished)
	{
		return;
	}

	if (URenderGridMoviePipelineRenderJob* Entry = URenderGridMoviePipelineRenderJob::Create(this, Job, Args); IsValid(Entry))
	{
		RemainingJobs.Add(Job);
		Entries.Add(Job, Entry);
	}
}

void URenderGridQueue::Pause()
{
	bPaused = true;
	if (bStarted && !bCanceled && !bFinished)
	{
		Queue->Stop();
	}
}

void URenderGridQueue::Resume()
{
	bPaused = false;
	if (bStarted && !bCanceled && !bFinished)
	{
		Queue->Start();
	}
}

void URenderGridQueue::Cancel()
{
	if (bCanceled || bFinished)
	{
		return;
	}
	bCanceled = true;

	TArray<TObjectPtr<URenderGridMoviePipelineRenderJob>> EntryValues;
	Entries.GenerateValueArray(EntryValues);
	for (int64 i = EntryValues.Num() - 1; i >= 0; i--)
	{
		if (IsValid(EntryValues[i]))
		{
			EntryValues[i]->Cancel();
		}
	}
}

URenderGridJob* URenderGridQueue::GetCurrentlyRenderingJob() const
{
	return CurrentJob;
}

FString URenderGridQueue::GetJobStatus(URenderGridJob* Job) const
{
	if (!IsValid(Job))
	{
		return TEXT("");
	}

	if (const TObjectPtr<URenderGridMoviePipelineRenderJob>* EntryPtr = Entries.Find(Job))
	{
		if (TObjectPtr<URenderGridMoviePipelineRenderJob> Entry = *EntryPtr; IsValid(Entry))
		{
			return Entry->GetStatus();
		}
	}
	return TEXT("");
}

float URenderGridQueue::GetJobStatusPercentage(URenderGridJob* Job) const
{
	if (!IsValid(Job))
	{
		return 0;
	}

	if (const TObjectPtr<URenderGridMoviePipelineRenderJob>* EntryPtr = Entries.Find(Job))
	{
		if (TObjectPtr<URenderGridMoviePipelineRenderJob> Entry = *EntryPtr; IsValid(Entry))
		{
			return Entry->GetStatusPercentage();
		}
	}
	return 0;
}

TArray<URenderGridJob*> URenderGridQueue::GetJobs() const
{
	TArray<URenderGridJob*> Jobs;
	Jobs.Reserve(Entries.Num());
	for (const TTuple<TObjectPtr<URenderGridJob>, TObjectPtr<URenderGridMoviePipelineRenderJob>>& EntryEntry : Entries)
	{
		if (const TObjectPtr<URenderGridJob> Job = EntryEntry.Key; IsValid(Job))
		{
			if (const TObjectPtr<URenderGridMoviePipelineRenderJob> Entry = EntryEntry.Value; IsValid(Entry))
			{
				if (Entry->CanExecute())
				{
					Jobs.Add(Job);
				}
			}
		}
	}
	return Jobs;
}

int32 URenderGridQueue::GetJobsCount() const
{
	int32 JobsCount = 0;
	for (const TTuple<TObjectPtr<URenderGridJob>, TObjectPtr<URenderGridMoviePipelineRenderJob>>& EntryEntry : Entries)
	{
		if (const TObjectPtr<URenderGridJob> Job = EntryEntry.Key; IsValid(Job))
		{
			if (const TObjectPtr<URenderGridMoviePipelineRenderJob> Entry = EntryEntry.Value; IsValid(Entry))
			{
				if (Entry->CanExecute())
				{
					JobsCount++;
				}
			}
		}
	}
	return JobsCount;
}

int32 URenderGridQueue::GetJobsRemainingCount() const
{
	if (bCanceled || bFinished)
	{
		return 0;
	}

	int32 JobsCount = 0;
	for (const TObjectPtr<URenderGridJob>& Job : RemainingJobs)
	{
		if (const TObjectPtr<URenderGridMoviePipelineRenderJob>* EntryPtr = Entries.Find(Job))
		{
			if (const TObjectPtr<URenderGridMoviePipelineRenderJob> Entry = *EntryPtr; IsValid(Entry))
			{
				if (Entry->CanExecute())
				{
					JobsCount++;
				}
			}
		}
	}
	return FMath::Min<int32>(GetJobsCount(), JobsCount + 1); // one is removed when it starts rendering, but this job is still remaining of course, so let's add 1 here
}

int32 URenderGridQueue::GetJobsCompletedCount() const
{
	return FMath::Max<int32>(0, GetJobsCount() - GetJobsRemainingCount());
}

FString URenderGridQueue::GetStatus() const
{
	if (bCanceled || bFinished)
	{
		return TEXT("Done");
	}
	if (!bStarted)
	{
		return TEXT("Waiting to be executed");
	}
	if (!IsCurrentlyRendering())
	{
		return TEXT("Waiting in queue");
	}
	if (bPaused)
	{
		return TEXT("Paused");
	}
	return TEXT("Rendering...");
}

float URenderGridQueue::GetStatusPercentage() const
{
	if (bCanceled || bFinished)
	{
		return 100.0;
	}

	double JobsCount = GetJobsCount();
	if (JobsCount <= 0)
	{
		return 0.0;
	}
	double RemainingCount = GetJobsRemainingCount();
	if (IsValid(CurrentEntry))
	{
		RemainingCount -= (CurrentEntry->GetStatusPercentage() / 100);
	}
	return FMath::Clamp(100.0 - ((RemainingCount / JobsCount) * 100), 0.0, 100.0);
}


void URenderGridQueue::OnStart()
{
	if (bStarted || bCanceled || bFinished)
	{
		return;
	}
	bStarted = true;

	OnExecuteStartedDelegate.Broadcast(this);
	OnExecuteStartedDelegate.Clear();
	AddToRoot();

	Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this]()
	{
		if (!RenderGrid->GetLevel().IsValid())
		{
			FEditorFileUtils::LoadMap(RenderGrid->GetLevel().ToString(), false, true);
		}
	}));
	Queue->DelayFrames(1);

	Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this]()
	{
		PreviousFrameLimitSettings = UE::RenderGrid::Private::FRenderGridUtils::DisableFpsLimit();
	}));
	Queue->DelayFrames(1);

	if (Args.bIsBatchRender)
	{
		Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this]()
		{
			if (IsValid(RenderGrid))
			{
				RenderGrid->BeginBatchRender(this);
			}
		}));
		Queue->DelayFrames(2);
	}

	OnProcessJob(nullptr); // starts the job execution queueing

	AddRenderingQueue(this);
}

void URenderGridQueue::OnProcessJob(URenderGridJob* Job)
{
	if (IsValid(Job) && !bCanceled && !bFinished)
	{
		if (TObjectPtr<URenderGridMoviePipelineRenderJob>* EntryPtr = Entries.Find(Job))
		{
			if (URenderGridMoviePipelineRenderJob* Entry = *EntryPtr; IsValid(Entry))
			{
				CurrentJob = Job;
				CurrentEntry = Entry;

				Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueActionReturningDelay::CreateLambda([this, Job, Entry]() -> UE::RenderGrid::Private::FRenderGridGenericExecutionQueueDelay
				{
					Entry->OnWaitForEngineWarmUpCount();
					if (IsValid(RenderGrid))
					{
						PreviousProps = UE::RenderGrid::IRenderGridModule::Get().GetManager().ApplyJobPropValues(RenderGrid, Job);
					}
					return UE::RenderGrid::Private::FRenderGridGenericExecutionQueueDelay::Frames(2 + (IsValid(RenderGrid) ? Job->GetWaitFramesBeforeRendering() : 0));
				}));

				Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this, Job, Entry]()
				{
					Entry->OnPreRenderEvent();
					if (IsValid(RenderGrid) && IsValid(Job))
					{
						RenderGrid->BeginJobRender(this, Job);
					}
				}));

				Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueActionReturningDelayFuture::CreateLambda([Entry]() -> TSharedFuture<void>
				{
					return Entry->Execute();
				}));

				Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this, Job, Entry]()
				{
					Entry->OnPostRenderEvent();
					if (IsValid(RenderGrid) && IsValid(Job))
					{
						RenderGrid->EndJobRender(this, Job);
					}
				}));

				Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueActionReturningDelay::CreateLambda([this]() -> UE::RenderGrid::Private::FRenderGridGenericExecutionQueueDelay
				{
					UE::RenderGrid::IRenderGridModule::Get().GetManager().RestorePropValues(PreviousProps);
					PreviousProps = FRenderGridManagerPreviousPropValues();
					return UE::RenderGrid::Private::FRenderGridGenericExecutionQueueDelay::Frames(2);
				}));

				Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this, Entry]()
				{
					if (Entry->IsCanceled())
					{
						Cancel();
					}
				}));
			}
		}
	}

	Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this]()
	{
		while (!RemainingJobs.IsEmpty() && !bCanceled && !bFinished)
		{
			TObjectPtr<URenderGridJob> NextJob = RemainingJobs[0];
			RemainingJobs.RemoveAt(0, EAllowShrinking::No);

			if (IsValid(NextJob))
			{
				OnProcessJob(NextJob);
				return;
			}
		}

		Queue->DelayFrames(2);
		Queue->Add(UE::RenderGrid::Private::FRenderGridGenericExecutionQueueAction::CreateLambda([this]()
		{
			OnFinish();
		}));
	}));
}

void URenderGridQueue::OnFinish()
{
	if (bFinished)
	{
		return;
	}
	bFinished = true; // prevents any new jobs from being added to it

	if (!bStarted)
	{
		return;
	}

	if (GetCurrentlyRenderingQueue() == this)
	{
		DoNextRenderingQueue(); // will start the next queue (during the next tick)
	}

	Queue->Stop();

	if (Args.bIsBatchRender && IsValid(RenderGrid))
	{
		RenderGrid->EndBatchRender(this);
	}

	UE::RenderGrid::Private::FRenderGridUtils::RestoreFpsLimit(PreviousFrameLimitSettings);
	PreviousFrameLimitSettings = FRenderGridPreviousEngineFpsSettings();

	RemoveFromRoot();

	OnExecuteFinishedDelegate.Broadcast(this, !bCanceled);
	OnExecuteFinishedDelegate.Clear();
}
