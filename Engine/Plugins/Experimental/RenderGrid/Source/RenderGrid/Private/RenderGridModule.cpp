// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGridModule.h"
#include "RenderGrid/RenderGridManager.h"
#include "RenderGrid/RenderGridPropsSource.h"
#include "Factories/RenderGridPropsSourceFactoryLocal.h"
#include "Factories/RenderGridPropsSourceFactoryRemoteControl.h"

#include "HAL/FileManager.h"

#define LOCTEXT_NAMESPACE "RenderGridModule"


void UE::RenderGrid::Private::FRenderGridModule::StartupModule()
{
	IFileManager::Get().DeleteDirectory(*FRenderGridManager::TmpRenderedFramesPath, false, true);
	RegisterPropsSourceFactories();
	CreateManager();
}

void UE::RenderGrid::Private::FRenderGridModule::ShutdownModule()
{
	RemoveManager();
	UnregisterPropsSourceFactories();
}


void UE::RenderGrid::Private::FRenderGridModule::CreateManager()
{
	Manager = MakeUnique<FRenderGridManager>();
}

void UE::RenderGrid::Private::FRenderGridModule::RemoveManager()
{
	Manager.Reset();
}

UE::RenderGrid::FRenderGridManager& UE::RenderGrid::Private::FRenderGridModule::GetManager() const
{
	check(Manager.IsValid());
	return *Manager;
}


void UE::RenderGrid::Private::FRenderGridModule::RegisterPropsSourceFactories()
{
	RegisterPropsSourceFactory(ERenderGridPropsSourceType::Local, MakeShared<FRenderGridPropsSourceFactoryLocal>());
	RegisterPropsSourceFactory(ERenderGridPropsSourceType::RemoteControl, MakeShared<FRenderGridPropsSourceFactoryRemoteControl>());
}

void UE::RenderGrid::Private::FRenderGridModule::UnregisterPropsSourceFactories()
{
	UnregisterPropsSourceFactory(ERenderGridPropsSourceType::Local);
	UnregisterPropsSourceFactory(ERenderGridPropsSourceType::RemoteControl);
}

void UE::RenderGrid::Private::FRenderGridModule::RegisterPropsSourceFactory(const ERenderGridPropsSourceType PropsSourceType, const TSharedPtr<IRenderGridPropsSourceFactory>& InFactory)
{
	PropsSourceFactories.Add(PropsSourceType, InFactory);
}

void UE::RenderGrid::Private::FRenderGridModule::UnregisterPropsSourceFactory(const ERenderGridPropsSourceType PropsSourceType)
{
	PropsSourceFactories.Remove(PropsSourceType);
}

URenderGridPropsSourceBase* UE::RenderGrid::Private::FRenderGridModule::CreatePropsSource(ERenderGridPropsSourceType PropsSourceType, UObject* PropsSourceOrigin)
{
	TSharedPtr<IRenderGridPropsSourceFactory>* FactoryPtr = PropsSourceFactories.Find(PropsSourceType);
	if (!FactoryPtr)
	{
		return nullptr;
	}

	TSharedPtr<IRenderGridPropsSourceFactory> Factory = *FactoryPtr;
	if (!Factory)
	{
		return nullptr;
	}

	return Factory->CreateInstance(PropsSourceOrigin);
}


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::RenderGrid::Private::FRenderGridModule, RenderGrid)
