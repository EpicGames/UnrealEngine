// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRenderGridModule.h"
#include "Factories/IRenderGridPropsSourceFactory.h"


namespace UE::RenderGrid::Private
{
	/**
	 * The implementation of the IRenderGridModule interface.
	 */
	class FRenderGridModule : public IRenderGridModule
	{
	public:
		//~ Begin IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface interface

		//~ Begin IRenderGridModule interface
		virtual FRenderGridManager& GetManager() const override;

		virtual URenderGridPropsSourceBase* CreatePropsSource(ERenderGridPropsSourceType PropsSourceType, UObject* PropsSourceOrigin) override;
		virtual const TMap<ERenderGridPropsSourceType, TSharedPtr<IRenderGridPropsSourceFactory>>& GetPropsSourceFactories() const override { return PropsSourceFactories; }
		//~ End IRenderGridModule interface

	private:
		void CreateManager();
		void RemoveManager();

		void RegisterPropsSourceFactories();
		void UnregisterPropsSourceFactories();
		void RegisterPropsSourceFactory(const ERenderGridPropsSourceType PropsSourceType, const TSharedPtr<IRenderGridPropsSourceFactory>& InFactory);
		void UnregisterPropsSourceFactory(const ERenderGridPropsSourceType PropsSourceType);

	private:
		TUniquePtr<FRenderGridManager> Manager;
		TMap<ERenderGridPropsSourceType, TSharedPtr<IRenderGridPropsSourceFactory>> PropsSourceFactories;
	};
}
