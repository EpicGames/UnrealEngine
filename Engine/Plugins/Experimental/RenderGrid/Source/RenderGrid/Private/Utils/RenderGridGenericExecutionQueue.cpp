// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/RenderGridGenericExecutionQueue.h"


void UE::RenderGrid::Private::FRenderGridGenericExecutionQueue::Tick(float DeltaTime)
{
	if (DelayRemainingFrames > 0)
	{
		DelayRemainingFrames--;
	}
	if (DelayRemainingSeconds > 0)
	{
		DelayRemainingSeconds -= DeltaTime;
	}

	if (DelayRemainingFuture.IsValid())
	{
		if (!DelayRemainingFuture.IsReady())
		{
			return;
		}
		DelayRemainingFuture = TFuture<void>();
	}
	if (DelayRemainingFutureReturningDelay.IsValid())
	{
		if (!DelayRemainingFutureReturningDelay.IsReady())
		{
			return;
		}
		QueueDelay(DelayRemainingFutureReturningDelay.Get());
		DelayRemainingFutureReturningDelay = TFuture<FRenderGridGenericExecutionQueueDelay>();
	}

	if (!bExecuting || ((DelayRemainingFrames <= 0) && (DelayRemainingSeconds <= 0)))
	{
		ExecuteNext();
	}
}

void UE::RenderGrid::Private::FRenderGridGenericExecutionQueue::Start()
{
	bRunning = true;
}

void UE::RenderGrid::Private::FRenderGridGenericExecutionQueue::Stop()
{
	bRunning = false;
}

void UE::RenderGrid::Private::FRenderGridGenericExecutionQueue::ExecuteNext()
{
	bExecuting = true;
	while (true)
	{
		if (DelayRemainingFuture.IsValid() || DelayRemainingFutureReturningDelay.IsValid())
		{
			// continue in Tick event
			return;
		}
		if (ExecuteNextDelay())
		{
			// continue in Tick event
			return;
		}
		if (bRunning && ExecuteNextEntry())
		{
			// execution completed, restart this loop  (to check for new delays, and to execute the next entry)
			continue;
		}
		break;
	}
	// if it gets to this point, it means that there are no delays or executions anymore  (since it would have returned out of this function otherwise)
	bExecuting = false;
}

bool UE::RenderGrid::Private::FRenderGridGenericExecutionQueue::ExecuteNextDelay()
{
	while (true)
	{
		if (TDoubleLinkedList<FRenderGridGenericExecutionQueueDelay>::TDoubleLinkedListNode* DelayNode = QueuedDelays.GetHead())
		{
			{// uses Delay before calling RemoveNode on it >>
				const FRenderGridGenericExecutionQueueDelay& Delay = DelayNode->GetValue();
				if (Delay.MinimumFrames > 0)
				{
					DelayRemainingFrames = Delay.MinimumFrames;
				}
				if (Delay.MinimumSeconds > 0)
				{
					DelayRemainingSeconds = Delay.MinimumSeconds;
				}
			}// uses Delay before calling RemoveNode on it <<

			QueuedDelays.RemoveNode(DelayNode);

			if ((DelayRemainingFrames <= 0) && (DelayRemainingSeconds <= 0))
			{
				continue;
			}
			return true;
		}
		return false;
	}
}

bool UE::RenderGrid::Private::FRenderGridGenericExecutionQueue::ExecuteNextEntry()
{
	if (TDoubleLinkedList<FRenderGridGenericExecutionQueueEntry>::TDoubleLinkedListNode* EntryNode = QueuedEntries.GetHead())
	{
		{// uses Entry before calling RemoveNode on it >>
			const FRenderGridGenericExecutionQueueEntry& Entry = EntryNode->GetValue();

			Entry.ActionRegular.ExecuteIfBound();

			if (Entry.ActionReturningDelay.IsBound())
			{
				QueueDelay(Entry.ActionReturningDelay.Execute());
			}

			if (Entry.ActionReturningDelayFuture.IsBound())
			{
				DelayRemainingFuture = Entry.ActionReturningDelayFuture.Execute();
			}

			if (Entry.ActionReturningDelayFutureReturningDelay.IsBound())
			{
				DelayRemainingFutureReturningDelay = Entry.ActionReturningDelayFutureReturningDelay.Execute();
			}
		}// uses Entry before calling RemoveNode on it <<

		QueuedEntries.RemoveNode(EntryNode);
		return true;
	}
	return false;
}
