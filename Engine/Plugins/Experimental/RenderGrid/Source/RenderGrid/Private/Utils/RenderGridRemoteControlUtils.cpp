// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/RenderGridRemoteControlUtils.h"

#include "StructUtils/InstancedStruct.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Utils/RenderGridUtils.h"
#include "Dom/JsonValue.h"
#include "Serialization/MemoryReader.h"
#include "UObject/StructOnScope.h"


URenderGridRemoteControlUtils::URenderGridRemoteControlUtils(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}


template<typename NumericType>
NumericType CastDoubleToNumericType(const double Value)
{
	if constexpr (std::is_same_v<NumericType, double>)
	{
		return Value;
	}
	else
	{
		if (Value < TNumericLimits<NumericType>::Lowest())
		{
			return TNumericLimits<NumericType>::Lowest();
		}
		if (Value > TNumericLimits<NumericType>::Max())
		{
			return TNumericLimits<NumericType>::Max();
		}
		return static_cast<NumericType>(Value);
	}
}


template<typename NumericType>
void ParseJsonAsNumericType(const FString& Json, const NumericType DefaultValue, NumericType& Value, bool& bSuccess)
{
	TSharedPtr<FJsonValue> JsonValue = UE::RenderGrid::Private::FRenderGridUtils::ParseJson(Json);
	if (!JsonValue.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	bSuccess = true;
	Value = CastDoubleToNumericType<NumericType>(JsonValue->AsNumber());
}

void ParseJsonAsStructType(const FString& Json, void* Value, UStruct& ValueTypeInfo, bool& bSuccess)
{
	FTCHARToUTF16 UTF16String(*Json, Json.Len());
	TArray<uint8> Bytes = TArray(reinterpret_cast<const uint8*>(UTF16String.Get()), UTF16String.Length() * sizeof(UTF16CHAR));

	FMemoryReader Reader = FMemoryReader(Bytes);
	FJsonStructDeserializerBackend ReaderBackend = FJsonStructDeserializerBackend(Reader);

	bSuccess = FStructDeserializer::Deserialize(Value, ValueTypeInfo, ReaderBackend);
	Reader.Close();
}

void ParseJsonAsStructType(const FString& Json, const FInstancedStruct& DefaultValue, FInstancedStruct& Value, bool& bSuccess)
{
	if (!DefaultValue.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(DefaultValue.GetScriptStruct()));
	UStruct* ValueTypeInfo = const_cast<UStruct*>(StructOnScope->GetStruct());
	ParseJsonAsStructType(Json, StructOnScope->GetStructMemory(), *ValueTypeInfo, bSuccess);
	if (!bSuccess)
	{
		Value = DefaultValue;
		return;
	}
	Value = FInstancedStruct();
	Value.InitializeAs(DefaultValue.GetScriptStruct(), StructOnScope->GetStructMemory());
}

template<typename StructType>
void ParseJsonAsStructType(const FString& Json, const StructType& DefaultValue, StructType& Value, bool& bSuccess)
{
	FInstancedStruct ValueWrapped = FInstancedStruct::Make<StructType>(Value);
	ParseJsonAsStructType(Json, FInstancedStruct::Make<StructType>(DefaultValue), ValueWrapped, bSuccess);
	Value = ValueWrapped.Get<StructType>();
}

void URenderGridRemoteControlUtils::ParseJsonAsByte(const FString& Json, const uint8 DefaultValue, bool& bSuccess, uint8& Value)
{
	ParseJsonAsNumericType<uint8>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsInt32(const FString& Json, const int32 DefaultValue, bool& bSuccess, int32& Value)
{
	ParseJsonAsNumericType<int32>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsInt64(const FString& Json, const int64 DefaultValue, bool& bSuccess, int64& Value)
{
	ParseJsonAsNumericType<int64>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsFloat(const FString& Json, const double DefaultValue, bool& bSuccess, double& Value)
{
	ParseJsonAsNumericType<double>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsBoolean(const FString& Json, const bool DefaultValue, bool& bSuccess, bool& Value)
{
	TSharedPtr<FJsonValue> JsonValue = UE::RenderGrid::Private::FRenderGridUtils::ParseJson(Json);
	if (!JsonValue.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	bSuccess = true;
	Value = JsonValue->AsBool();
}

void URenderGridRemoteControlUtils::ParseJsonAsString(const FString& Json, const FString& DefaultValue, bool& bSuccess, FString& Value)
{
	TSharedPtr<FJsonValue> JsonValue = UE::RenderGrid::Private::FRenderGridUtils::ParseJson(Json);
	if (!JsonValue.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	bSuccess = true;
	Value = JsonValue->AsString();
}

void URenderGridRemoteControlUtils::ParseJsonAsName(const FString& Json, const FName& DefaultValue, bool& bSuccess, FName& Value)
{
	FString ValueString;
	ParseJsonAsString(Json, DefaultValue.ToString(), bSuccess, ValueString);
	Value = FName(ValueString);
}

void URenderGridRemoteControlUtils::ParseJsonAsText(const FString& Json, const FText& DefaultValue, bool& bSuccess, FText& Value)
{
	TSharedPtr<FJsonValue> JsonValue = UE::RenderGrid::Private::FRenderGridUtils::ParseJson(Json);
	if (!JsonValue.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	FText TextValue;
	if (!FTextStringHelper::ReadFromBuffer(*JsonValue->AsString(), TextValue))
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	bSuccess = true;
	Value = TextValue;
}

void URenderGridRemoteControlUtils::ParseJsonAsObjectReference(const FString& Json, UObject* DefaultValue, bool& bSuccess, UObject*& Value)
{
	FString ObjectPath;
	ParseJsonAsString(Json, TEXT(""), bSuccess, ObjectPath);
	if (!bSuccess || ObjectPath.IsEmpty())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	FSoftObjectPath SoftObjectPath = FSoftObjectPath(ObjectPath);
	if (!SoftObjectPath.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	Value = TSoftObjectPtr(SoftObjectPath).LoadSynchronous();
	if (!IsValid(Value))
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	bSuccess = true;
}

void URenderGridRemoteControlUtils::ParseJsonAsClassReference(const FString& Json, UClass* DefaultValue, bool& bSuccess, UClass*& Value)
{
	FString ClassPath;
	ParseJsonAsString(Json, TEXT(""), bSuccess, ClassPath);
	if (!bSuccess || ClassPath.IsEmpty())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	FSoftClassPath SoftClassPath = FSoftClassPath(ClassPath);
	if (!SoftClassPath.IsValid())
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	Value = TSoftClassPtr(SoftClassPath).LoadSynchronous();
	if (!IsValid(Value))
	{
		bSuccess = false;
		Value = DefaultValue;
		return;
	}
	bSuccess = true;
}

void URenderGridRemoteControlUtils::ParseJsonAsStruct(const FString& Json, const FInstancedStruct& DefaultValue, bool& bSuccess, FInstancedStruct& Value)
{
	ParseJsonAsStructType(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsVector(const FString& Json, const FVector& DefaultValue, bool& bSuccess, FVector& Value)
{
	ParseJsonAsStructType<FVector>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsRotator(const FString& Json, const FRotator& DefaultValue, bool& bSuccess, FRotator& Value)
{
	ParseJsonAsStructType<FRotator>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsTransform(const FString& Json, const FTransform& DefaultValue, bool& bSuccess, FTransform& Value)
{
	ParseJsonAsStructType<FTransform>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsColor(const FString& Json, const FColor& DefaultValue, bool& bSuccess, FColor& Value)
{
	ParseJsonAsStructType<FColor>(Json, DefaultValue, Value, bSuccess);
}

void URenderGridRemoteControlUtils::ParseJsonAsLinearColor(const FString& Json, const FLinearColor& DefaultValue, bool& bSuccess, FLinearColor& Value)
{
	ParseJsonAsStructType<FLinearColor>(Json, DefaultValue, Value, bSuccess);
}


void StructTypeToJson(const void* Value, UStruct& ValueTypeInfo, FString& Json)
{
	TArray<uint8> Bytes;
	FMemoryWriter Writer = FMemoryWriter(Bytes);
	FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);

	FStructSerializer::Serialize(Value, ValueTypeInfo, WriterBackend);
	Writer.Close();

	const FUTF16ToTCHAR ConvertedString(reinterpret_cast<UTF16CHAR*>(Bytes.GetData()), Bytes.Num() / sizeof(UTF16CHAR));
	Json = FString::ConstructFromPtrSize(ConvertedString.Get(), ConvertedString.Length()).TrimStartAndEnd();
}

void StructTypeToJson(const FInstancedStruct& Value, FString& Json)
{
	if (!Value.IsValid())
	{
		Json = TEXT("");
		return;
	}
	UStruct* ValueTypeInfo = const_cast<UScriptStruct*>(Value.GetScriptStruct());
	StructTypeToJson(Value.GetMemory(), *ValueTypeInfo, Json);
}

template<typename StructType>
void StructTypeToJson(const StructType& Value, FString& Json)
{
	StructTypeToJson(FInstancedStruct::Make<StructType>(Value), Json);
}

void URenderGridRemoteControlUtils::ByteToJson(const uint8 Value, FString& Json)
{
	Json = UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(MakeShareable(new FJsonValueNumber(Value)));
}

void URenderGridRemoteControlUtils::Int32ToJson(const int32 Value, FString& Json)
{
	Json = UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(MakeShareable(new FJsonValueNumber(Value)));
}

void URenderGridRemoteControlUtils::Int64ToJson(const int64 Value, FString& Json)
{
	Json = UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(MakeShareable(new FJsonValueNumber(Value)));
}

void URenderGridRemoteControlUtils::FloatToJson(const double Value, FString& Json)
{
	Json = UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(MakeShareable(new FJsonValueNumber(Value)));
}

void URenderGridRemoteControlUtils::BooleanToJson(const bool Value, FString& Json)
{
	Json = UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(MakeShareable(new FJsonValueBoolean(Value)));
}

void URenderGridRemoteControlUtils::StringToJson(const FString& Value, FString& Json)
{
	Json = UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(MakeShareable(new FJsonValueString(Value)));
}

void URenderGridRemoteControlUtils::NameToJson(const FName& Value, FString& Json)
{
	StringToJson(Value.ToString(), Json);
}

void URenderGridRemoteControlUtils::TextToJson(const FText& Value, FString& Json)
{
	FString ValueAsString;
	FTextStringHelper::WriteToBuffer(ValueAsString, Value);
	StringToJson(ValueAsString, Json);
}

void URenderGridRemoteControlUtils::ObjectReferenceToJson(UObject* Value, FString& Json)
{
	StringToJson(GetPathNameSafe(Value), Json);
}

void URenderGridRemoteControlUtils::ClassReferenceToJson(UClass* Value, FString& Json)
{
	StringToJson(GetPathNameSafe(Value), Json);
}

void URenderGridRemoteControlUtils::StructToJson(const FInstancedStruct& Value, FString& Json)
{
	StructTypeToJson(Value, Json);
}

void URenderGridRemoteControlUtils::VectorToJson(const FVector& Value, FString& Json)
{
	StructTypeToJson<FVector>(Value, Json);
}

void URenderGridRemoteControlUtils::RotatorToJson(const FRotator& Value, FString& Json)
{
	StructTypeToJson<FRotator>(Value, Json);
}

void URenderGridRemoteControlUtils::TransformToJson(const FTransform& Value, FString& Json)
{
	StructTypeToJson<FTransform>(Value, Json);
}

void URenderGridRemoteControlUtils::ColorToJson(const FColor& Value, FString& Json)
{
	StructTypeToJson<FColor>(Value, Json);
}

void URenderGridRemoteControlUtils::LinearColorToJson(const FLinearColor& Value, FString& Json)
{
	StructTypeToJson<FLinearColor>(Value, Json);
}
