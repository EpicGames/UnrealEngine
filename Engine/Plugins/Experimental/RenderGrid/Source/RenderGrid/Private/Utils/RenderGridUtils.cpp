// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/RenderGridUtils.h"
#include "RenderGrid/RenderGrid.h"
#include "Engine/Texture2D.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "TextureResource.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "RenderGridLog.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"


bool UE::RenderGrid::Private::FRenderGridUtils::IsImage(const FString& ImagePath)
{
	static IImageWrapperModule* ImageWrapperModule = FModuleManager::LoadModulePtr<IImageWrapperModule>("ImageWrapper");
	if (ImageWrapperModule == nullptr)
	{
		return false;
	}

	EImageFormat ImageFormat = ImageWrapperModule->GetImageFormatFromExtension(*ImagePath);
	if (ImageFormat == EImageFormat::Invalid)
	{
		return false;
	}

	return true;
}

UTexture2D* UE::RenderGrid::Private::FRenderGridUtils::GetImage(const FString& ImagePath, UTexture2D* Texture2D, bool& bOutReusedGivenTexture2D)
{
	static IImageWrapperModule* ImageWrapperModule = FModuleManager::LoadModulePtr<IImageWrapperModule>("ImageWrapper");
	if (ImageWrapperModule == nullptr)
	{
		bOutReusedGivenTexture2D = (Texture2D == nullptr);
		return nullptr;
	}

	EImageFormat ImageFormat = ImageWrapperModule->GetImageFormatFromExtension(*ImagePath);
	if (ImageFormat == EImageFormat::Invalid)
	{
		bOutReusedGivenTexture2D = (Texture2D == nullptr);
		return nullptr;
	}

	TArray<uint8> ImageBytes = GetFileData(ImagePath);
	if (ImageBytes.Num() <= 0)
	{
		bOutReusedGivenTexture2D = (Texture2D == nullptr);
		return nullptr;
	}

	return BytesToExistingImage(ImageBytes, ImageFormat, Texture2D, bOutReusedGivenTexture2D);
}


UTexture2D* UE::RenderGrid::Private::FRenderGridUtils::BytesToImage(const TArray<uint8>& ByteArray, const EImageFormat ImageFormat)
{
	static IImageWrapperModule* ImageWrapperModule = FModuleManager::LoadModulePtr<IImageWrapperModule>("ImageWrapper");
	if (ImageWrapperModule == nullptr)
	{
		return nullptr;
	}

	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule->CreateImageWrapper(ImageFormat);
	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(ByteArray.GetData(), ByteArray.Num()))
	{
		return nullptr;
	}

	TArray<uint8> Uncompressed;
	if (!ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, Uncompressed))
	{
		return nullptr;
	}

	return DataToTexture2D(ImageWrapper->GetWidth(), ImageWrapper->GetHeight(), Uncompressed.GetData(), Uncompressed.Num());
}

UTexture2D* UE::RenderGrid::Private::FRenderGridUtils::BytesToExistingImage(const TArray<uint8>& ByteArray, const EImageFormat ImageFormat, UTexture2D* Texture2D, bool& bOutReusedGivenTexture2D)
{
	static IImageWrapperModule* ImageWrapperModule = FModuleManager::LoadModulePtr<IImageWrapperModule>("ImageWrapper");
	if (ImageWrapperModule == nullptr)
	{
		bOutReusedGivenTexture2D = (Texture2D == nullptr);
		return nullptr;
	}

	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule->CreateImageWrapper(ImageFormat);
	if (!ImageWrapper.IsValid() || !ImageWrapper->SetCompressed(ByteArray.GetData(), ByteArray.Num()))
	{
		bOutReusedGivenTexture2D = (Texture2D == nullptr);
		return nullptr;
	}

	TArray<uint8> Uncompressed;
	if (!ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, Uncompressed))
	{
		bOutReusedGivenTexture2D = (Texture2D == nullptr);
		return nullptr;
	}

	UTexture2D* NewTexture2D = DataToExistingTexture2D(ImageWrapper->GetWidth(), ImageWrapper->GetHeight(), Uncompressed.GetData(), Uncompressed.Num(), Texture2D, bOutReusedGivenTexture2D);
	bOutReusedGivenTexture2D = (Texture2D == NewTexture2D);
	return NewTexture2D;
}


UTexture2D* UE::RenderGrid::Private::FRenderGridUtils::DataToTexture2D(int32 Width, int32 Height, const void* Src, SIZE_T Count)
{
	UTexture2D* Texture2D = UTexture2D::CreateTransient(Width, Height, PF_B8G8R8A8);
	if (Texture2D == nullptr)
	{
		return nullptr;
	}
	Texture2D->bNoTiling = true;

#if WITH_EDITORONLY_DATA
	Texture2D->MipGenSettings = TMGS_NoMipmaps;
#endif

	void* TextureData = Texture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);
	FMemory::Memcpy(TextureData, Src, Count);
	Texture2D->GetPlatformData()->Mips[0].BulkData.Unlock();

	Texture2D->UpdateResource();
	return Texture2D;
}

UTexture2D* UE::RenderGrid::Private::FRenderGridUtils::DataToExistingTexture2D(int32 Width, int32 Height, const void* Src, SIZE_T Count, UTexture2D* Texture2D, bool& bOutReusedGivenTexture2D)
{
	UTexture2D* OriginalTexture2D = Texture2D;

	bOutReusedGivenTexture2D = true;
	if ((Texture2D == nullptr) || (Texture2D->GetSizeX() != Width) || (Texture2D->GetSizeY() != Height) || (Texture2D->GetPixelFormat() != PF_B8G8R8A8))
	{
		bOutReusedGivenTexture2D = false;
		Texture2D = UTexture2D::CreateTransient(Width, Height, PF_B8G8R8A8);
	}
	if (Texture2D == nullptr)
	{
		bOutReusedGivenTexture2D = (OriginalTexture2D == nullptr);
		return nullptr;
	}
	Texture2D->bNoTiling = true;

#if WITH_EDITORONLY_DATA
	Texture2D->MipGenSettings = TMGS_NoMipmaps;
#endif

	void* TextureData = Texture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);
	FMemory::Memcpy(TextureData, Src, Count);
	Texture2D->GetPlatformData()->Mips[0].BulkData.Unlock();

	Texture2D->UpdateResource();
	return Texture2D;
}


TArray<FString> UE::RenderGrid::Private::FRenderGridUtils::GetFiles(const FString& Directory, const bool bRecursive)
{
	class FRenderGridUtilsGetFilesVisitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		TArray<FString> Files;
		virtual bool Visit(const TCHAR* FileName, const bool bIsDirectory) override
		{
			if (!bIsDirectory)
			{
				Files.Add(FileName);
			}
			return true;
		}
	};
	FRenderGridUtilsGetFilesVisitor GetFilesVisitor;
	if (bRecursive)
	{
		FPlatformFileManager::Get().GetPlatformFile().IterateDirectoryRecursively(*Directory, GetFilesVisitor);
	}
	else
	{
		FPlatformFileManager::Get().GetPlatformFile().IterateDirectory(*Directory, GetFilesVisitor);
	}
	return GetFilesVisitor.Files;
}

TArray<uint8> UE::RenderGrid::Private::FRenderGridUtils::GetFileData(const FString& File)
{
	TArray<uint8> Data;
	IFileHandle* FileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenRead(*File);
	if (FileHandle != nullptr)
	{
		int64 FileSize = FileHandle->Size();
		if (FileSize > 0x7fffffff)
		{
			FileSize = 0;
		}
		if (FileSize > 0)
		{
			Data.SetNum(FileSize);
			FileHandle->Read(Data.GetData(), FileSize);
		}
		delete FileHandle;
	}
	return Data;
}


void UE::RenderGrid::Private::FRenderGridUtils::DeleteDirectory(const FString& Directory)
{
	IFileManager::Get().DeleteDirectory(*Directory, false, true);
}

void UE::RenderGrid::Private::FRenderGridUtils::EmptyDirectory(const FString& Directory)
{
	class FRenderGridUtilsEmptyDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		virtual bool Visit(const TCHAR* FileName, const bool bIsDirectory) override
		{
			if (bIsDirectory)
			{
				IFileManager::Get().DeleteDirectory(FileName, false, false);
			}
			else
			{
				IFileManager::Get().Delete(FileName, false, true, true);
			}
			return true;
		}
	};
	FRenderGridUtilsEmptyDirectoryVisitor GetFilesVisitor;
	FPlatformFileManager::Get().GetPlatformFile().IterateDirectoryRecursively(*Directory, GetFilesVisitor);
}


FString UE::RenderGrid::Private::FRenderGridUtils::NormalizeOutputDirectory(const FString& OutputDirectory)
{
	FString NewOutputDirectory = OutputDirectory;
	FPaths::NormalizeDirectoryName(NewOutputDirectory);
	FPaths::RemoveDuplicateSlashes(NewOutputDirectory);
	FPaths::CollapseRelativeDirectories(NewOutputDirectory);
	FPaths::RemoveDuplicateSlashes(NewOutputDirectory);
	if (!NewOutputDirectory.EndsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		NewOutputDirectory += TEXT("/");
	}
	return NewOutputDirectory;
}


FString UE::RenderGrid::Private::FRenderGridUtils::PurgeJobIdOrReturnEmptyString(const FString& JobId)
{
	static FString ValidCharacters = TEXT("abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789-_");

	FString Result;
	for (TCHAR JobIdChar : JobId)
	{
		int32 Index;
		if (ValidCharacters.FindChar(JobIdChar, Index))
		{
			Result += JobIdChar;
		}
	}
	return Result;
}

FString UE::RenderGrid::Private::FRenderGridUtils::PurgeJobId(const FString& JobId)
{
	FString Result = PurgeJobIdOrReturnEmptyString(JobId);
	if (Result.IsEmpty())
	{
		return TEXT("0");
	}
	return Result;
}

FString UE::RenderGrid::Private::FRenderGridUtils::PurgeJobIdOrGenerateUniqueId(URenderGrid* Grid, const FString& JobId)
{
	FString Result = PurgeJobIdOrReturnEmptyString(JobId);
	if (Result.IsEmpty())
	{
		return Grid->GenerateNextJobId();
	}
	return Result;
}

FString UE::RenderGrid::Private::FRenderGridUtils::PurgeJobName(const FString& JobName)
{
	return JobName.TrimStartAndEnd();
}

FString UE::RenderGrid::Private::FRenderGridUtils::NormalizeJobOutputDirectory(const FString& OutputDirectory)
{
	return NormalizeOutputDirectory(FPaths::ConvertRelativePathToFull(OutputDirectory))
		.Replace(*NormalizeOutputDirectory(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir())), TEXT("{project_dir}/"));
}

FString UE::RenderGrid::Private::FRenderGridUtils::DenormalizeJobOutputDirectory(const FString& NormalizedOutputDirectory)
{
	return NormalizeOutputDirectory(FPaths::ConvertRelativePathToFull(
		NormalizedOutputDirectory.Replace(TEXT("{project_dir}"), *FPaths::ProjectDir())
	));
}


FString UE::RenderGrid::Private::FRenderGridUtils::ToJsonString(const TSharedPtr<FJsonValue>& Value, const bool PrettyPrint)
{
	if (!Value.IsValid())
	{
		return TEXT("NULL");
	}

	FString JsonString;
	if (PrettyPrint)
	{
		TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonString);
		FJsonSerializer::Serialize({Value}, JsonWriter);
		JsonWriter->Close();
	}
	else
	{
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonString);
		FJsonSerializer::Serialize({Value}, JsonWriter);
		JsonWriter->Close();
	}

	JsonString.TrimStartAndEndInline();
	JsonString.MidInline(1, JsonString.Len() - 2);
	JsonString.TrimStartAndEndInline();
	return JsonString;
}

TSharedPtr<FJsonValue> UE::RenderGrid::Private::FRenderGridUtils::ParseJson(const FString& Json)
{
	const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(TEXT("{\"value\":") + Json + TEXT("}"));
	TSharedPtr<FJsonObject> JsonObjectData;
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObjectData))
	{
		// the given string isn't valid json
		return TSharedPtr<FJsonValue>();
	}
	return JsonObjectData->TryGetField(TEXT("value"));
}


FString UE::RenderGrid::Private::FRenderGridUtils::GetRemoteControlValueJsonFromBytes(const TArray<uint8>& ValueBytes, const bool PrettyPrint)
{
	TSharedPtr<FJsonValue> Value;
	{
		const FUTF16ToTCHAR ConvertedString(reinterpret_cast<UTF16CHAR*>(const_cast<uint8*>(ValueBytes.GetData())), ValueBytes.Num() / sizeof(UTF16CHAR));
		const FString Json = FString::ConstructFromPtrSize(ConvertedString.Get(), ConvertedString.Length()).TrimStartAndEnd();

		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(Json);
		TSharedPtr<FJsonObject> JsonObjectData;
		if (!FJsonSerializer::Deserialize(JsonReader, JsonObjectData))
		{
			// the given bytes isn't valid json
			return TEXT("NULL");
		}

		TArray<TSharedPtr<FJsonValue>> Values;
		JsonObjectData->Values.GenerateValueArray(Values);
		if (Values.Num() != 1)
		{
			// in the given bytes (json data), there is no key, or there are multiple keys, either way it's invalid
			return TEXT("NULL");
		}

		Value = Values[0];
	}

	return ToJsonString(Value, PrettyPrint);
}

TArray<uint8> UE::RenderGrid::Private::FRenderGridUtils::GetRemoteControlValueBytesFromJson(const TArray<uint8>& OldValueBytes, const FString& NewValueJson)
{
	UE::FSharedString OldValueKey;
	{
		const FUTF16ToTCHAR ConvertedString(reinterpret_cast<UTF16CHAR*>(const_cast<uint8*>(OldValueBytes.GetData())), OldValueBytes.Num() / sizeof(UTF16CHAR));
		const FString Json = FString::ConstructFromPtrSize(ConvertedString.Get(), ConvertedString.Length());

		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(Json);
		TSharedPtr<FJsonObject> JsonObjectData;
		if (!FJsonSerializer::Deserialize(JsonReader, JsonObjectData))
		{
			// the given bytes isn't valid json
			return TArray<uint8>();
		}

		TArray<FJsonObject::FStringType> Keys;
		JsonObjectData->Values.GenerateKeyArray(Keys);
		if (Keys.Num() != 1)
		{
			// in the given bytes (json data), there is no key, or there are multiple keys, either way it's invalid
			return TArray<uint8>();
		}

		OldValueKey = Keys[0];
	}

	TSharedPtr<FJsonValue> Value = ParseJson(NewValueJson);
	if (!Value.IsValid())
	{
		UE_LOGF(LogRenderGrid, Warning, "Unable to retrieve remote control value from provided json. The json is invalid: '%ls'", *NewValueJson);
		return TArray<uint8>();
	}

	FString JsonString;
	{
		TSharedRef<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
		JsonObject->SetField(OldValueKey, Value);

		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonString);
		FJsonSerializer::Serialize(JsonObject, JsonWriter);
		JsonWriter->Close();
	}

	FTCHARToUTF16 UTF16String(*JsonString, JsonString.Len());
	return TArray(reinterpret_cast<const uint8*>(UTF16String.Get()), UTF16String.Length() * sizeof(UTF16CHAR));
}


FRenderGridPreviousEngineFpsSettings UE::RenderGrid::Private::FRenderGridUtils::DisableFpsLimit()
{
	FRenderGridPreviousEngineFpsSettings Settings;
	if (GEngine)
	{
		static IConsoleVariable* VSync = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
		static IConsoleVariable* VSyncEditor = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSyncEditor"));
		UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

		Settings.bHasBeenSet = true;
		Settings.bUseFixedFrameRate = GEngine->bUseFixedFrameRate;
		Settings.bForceDisableFrameRateSmoothing = GEngine->bForceDisableFrameRateSmoothing;
		Settings.MaxFps = GEngine->GetMaxFPS();
		Settings.bVSync = VSync->GetBool();
		Settings.bVSyncEditor = VSyncEditor->GetBool();
		Settings.bThrottleCPUWhenNotForeground = EditorPerformanceSettings->bThrottleCPUWhenNotForeground;

		GEngine->bUseFixedFrameRate = false;
		GEngine->bForceDisableFrameRateSmoothing = true;
		GEngine->SetMaxFPS(10000);
		VSync->Set(false);
		VSyncEditor->Set(false);
		EditorPerformanceSettings->bThrottleCPUWhenNotForeground = false;
		EditorPerformanceSettings->PostEditChange();
	}
	return Settings;
}

void UE::RenderGrid::Private::FRenderGridUtils::RestoreFpsLimit(const FRenderGridPreviousEngineFpsSettings& Settings)
{
	if (GEngine && Settings.bHasBeenSet)
	{
		static IConsoleVariable* VSync = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
		static IConsoleVariable* VSyncEditor = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSyncEditor"));
		UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

		GEngine->bUseFixedFrameRate = Settings.bUseFixedFrameRate;
		GEngine->bForceDisableFrameRateSmoothing = Settings.bForceDisableFrameRateSmoothing;
		GEngine->SetMaxFPS(Settings.MaxFps);
		VSync->Set(Settings.bVSync);
		VSyncEditor->Set(Settings.bVSyncEditor);
		EditorPerformanceSettings->bThrottleCPUWhenNotForeground = Settings.bThrottleCPUWhenNotForeground;
		EditorPerformanceSettings->PostEditChange();
	}
}
