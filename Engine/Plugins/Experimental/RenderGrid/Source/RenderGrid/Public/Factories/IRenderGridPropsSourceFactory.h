// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class URenderGridPropsSourceBase;


namespace UE::RenderGrid
{
	/**
	 * The base class for the factory classes that will create URenderGridPropsSourceBase instances.
	 */
	class RENDERGRID_API IRenderGridPropsSourceFactory
	{
	public:
		virtual ~IRenderGridPropsSourceFactory() = default;
		virtual URenderGridPropsSourceBase* CreateInstance(UObject* PropsSourceOrigin) { return nullptr; }
	};
}
