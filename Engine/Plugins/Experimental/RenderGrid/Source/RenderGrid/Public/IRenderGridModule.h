// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"


class URenderGridPropsSourceBase;
enum class ERenderGridPropsSourceType : uint8;

namespace UE::RenderGrid
{
	class FRenderGridManager;
	class IRenderGridPropsSourceFactory;
}


namespace UE::RenderGrid
{
	/**
	* RenderGrid module interface.
	*/
	class IRenderGridModule : public IModuleInterface
	{
	public:
		/**
		 * Singleton-like access to IRenderGridModule.
		 *
		 * @return Returns RenderGridModule singleton instance, loading the module on demand if needed.
		 */
		static IRenderGridModule& Get()
		{
			static const FName ModuleName = "RenderGrid";
			return FModuleManager::LoadModuleChecked<IRenderGridModule>(ModuleName);
		}

		/**
		 * Singleton-like access to FRenderGridManager. Will error if this module hasn't started (or has stopped).
		 *
		 * @return Returns RenderGridManager singleton instance.
		 */
		virtual FRenderGridManager& GetManager() const = 0;

		/** Creates a URenderGridPropsSourceBase instance, based on the given ERenderGridPropsSourceType. */
		virtual URenderGridPropsSourceBase* CreatePropsSource(ERenderGridPropsSourceType PropsSourceType, UObject* PropsSourceOrigin) = 0;

		/** Returns all set IRenderGridPropsSourceFactory instances, these are used to create URenderGridPropsSourceBase instances based on a given ERenderGridPropsSourceType. */
		virtual const TMap<ERenderGridPropsSourceType, TSharedPtr<IRenderGridPropsSourceFactory>>& GetPropsSourceFactories() const = 0;
	};
}
