// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGrid/RenderGridPropsSource.h"
#include "RenderGrid.generated.h"


class FJsonValue;
class ULevelSequence;
class UMoviePipelineOutputSetting;
class UMoviePipelinePrimaryConfig;
class URenderGrid;
class URenderGridQueue;


/**
 * This struct contains the data for a certain remote control property.
 * 
 * It's currently simply a wrapper around a byte array.
 * This struct is needed so that that byte array can be used in another UPROPERTY container (TMap, TArray, etc).
 */
USTRUCT(BlueprintType)
struct RENDERGRID_API FRenderGridRemoteControlPropertyData
{
	GENERATED_BODY()

public:
	FRenderGridRemoteControlPropertyData() = default;
	FRenderGridRemoteControlPropertyData(const TArray<uint8>& InBytes)
		: Bytes(InBytes)
	{}

public:
	/** The property data, as bytes. */
	UPROPERTY()
	TArray<uint8> Bytes;
};


/**
 * This class contains the default values of render grid jobs.
 *
 * This is placed in a separate class, primarily so that it be separated easily from the render grid when using the details view widget.
 */
UCLASS()
class RENDERGRID_API URenderGridSettings : public UObject
{
	GENERATED_BODY()

public:
	URenderGridSettings();

	void CopyValuesFrom(URenderGridSettings* From)
	{
		Level = From->Level;
		PropsSourceType = From->PropsSourceType;
		PropsSourceOrigin_RemoteControl = From->PropsSourceOrigin_RemoteControl;
	}

public:
	UPROPERTY(EditInstanceOnly, Category="Render Grid", DisplayName="Level", Meta=(AllowedClasses="/Script/Engine.World"))
	TSoftObjectPtr<UWorld> Level;

	/** The type of the properties that a job in this grid can have. */
	UPROPERTY(/*EditInstanceOnly, Category="Render Grid", Meta=(DisplayName="Properties Type")*/)
	ERenderGridPropsSourceType PropsSourceType;

	/** The remote control properties that a job in this grid can have, only use this if PropsSourceType is ERenderGridPropsSourceType::RemoteControl. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid", Meta=(DisplayName="Remote Control Preset" /*, EditCondition="PropsSourceType == ERenderGridPropsSourceType::RemoteControl", EditConditionHides*/))
	TObjectPtr<URemoteControlPreset> PropsSourceOrigin_RemoteControl;

public:
	/** GetPropsSource calls are somewhat expensive, we speed that up by caching the result (the PropsSource) that has been last outputted by that function. */
	UPROPERTY(Transient)
	mutable TObjectPtr<URenderGridPropsSourceBase> CachedPropsSource;

	/** GetPropsSource calls are somewhat expensive, we speed that up by caching the PropsSourceType last used in that function. */
	UPROPERTY(Transient)
	mutable ERenderGridPropsSourceType CachedPropsSourceType;

	/** GetPropsSource calls are somewhat expensive, we speed that up by caching the PropsSourceOrigin last used in that function. */
	UPROPERTY(Transient)
	mutable TWeakObjectPtr<UObject> CachedPropsSourceOriginWeakPtr;
};


/**
 * This class contains the default values of render grid jobs.
 *
 * This is placed in a separate class, primarily so that it be separated easily from the render grid when using the details view widget.
 */
UCLASS()
class RENDERGRID_API URenderGridDefaults : public UObject
{
	GENERATED_BODY()

public:
	URenderGridDefaults();

	void CopyValuesFrom(URenderGridDefaults* From)
	{
		LevelSequence = From->LevelSequence;
		RenderPreset = From->RenderPreset;
		OutputDirectory = From->OutputDirectory;
	}

public:
	/** The default level sequence for new jobs, this is what will be rendered during rendering. A job without a level sequence can't be rendered. */
	UPROPERTY(EditInstanceOnly, Category="Defaults", Meta=(DisplayName="Default Level Sequence"))
	TObjectPtr<ULevelSequence> LevelSequence;

	/** The default movie pipeline render preset for new jobs. Render grid jobs are rendered using the movie pipeline plugin. This 'preset' contains the configuration of that plugin. */
	UPROPERTY(EditInstanceOnly, Category="Defaults", Meta=(DisplayName="Default Render Preset"))
	TObjectPtr<UMoviePipelinePrimaryConfig> RenderPreset;

	/** The default output directory for new jobs. This is the folder in which the output files (of rendering) are placed into. To be more specific, the output files are placed in: {OutputDirectory}/{JobId}/, this folder will be created if it doesn't exist at the time of rendering. */
	UPROPERTY(EditInstanceOnly, Category="Defaults", Meta=(DisplayName="Default Output Directory"))
	FString OutputDirectory;
};


/**
 * This class represents a render grid job, in other words, an entry (a row) of a render grid.
 * It contains a level sequence and custom properties that will be applied while rendering.
 * 
 * Each RenderGridJob must belong to a RenderGrid.
 */
UCLASS(BlueprintType, Meta=(DontUseGenericSpawnObject="true"))
class RENDERGRID_API URenderGridJob : public UObject
{
	GENERATED_BODY()

public:
	URenderGridJob();

	/** Obtains a string representation of this object. Shouldn't be used for anything other than logging/debugging. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job", Meta=(DisplayName="To Debug String (Render Grid Job)", CompactNodeTitle="DEBUG"))
	FString ToDebugString() const;

	/** Obtains a JSON representation of this object. Shouldn't be used for anything other than logging/debugging. */
	TSharedPtr<FJsonValue> ToDebugJson() const;

public:
	/** Gets the calculated start frame, not taking the framerate of the render preset into account. */
	TOptional<int32> GetSequenceStartFrame() const;

	/** Gets the calculated end frame, not taking the framerate of the render preset into account. */
	TOptional<int32> GetSequenceEndFrame() const;

	/** Sets the custom start frame to match the given sequence start frame. */
	bool SetSequenceStartFrame(const int32 NewCustomStartFrame);

	/** Sets the custom end frame to match the given sequence end frame. */
	bool SetSequenceEndFrame(const int32 NewCustomEndFrame);


	/** Gets the calculated start frame, if possible. */
	TOptional<int32> GetStartFrame() const;

	/** Gets the calculated end frame, if possible. */
	TOptional<int32> GetEndFrame() const;

	/** Gets the calculated start time (in seconds), if possible. */
	TOptional<double> GetStartTime() const;

	/** Gets the calculated end time (in seconds), if possible. */
	TOptional<double> GetEndTime() const;

	/** Gets the calculated duration (in seconds), if possible. */
	TOptional<double> GetDuration() const;

public:
	/** Gets the calculated start frame, if possible. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	void GetStartFrame(bool& bSuccess, int32& StartFrame) const;

	/** Gets the calculated end frame, if possible. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	void GetEndFrame(bool& bSuccess, int32& EndFrame) const;

	/** Gets the calculated start time, if possible. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	void GetStartTime(bool& bSuccess, double& StartTime) const;

	/** Gets the calculated end time, if possible. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	void GetEndTime(bool& bSuccess, double& EndTime) const;

	/** Gets the calculated duration in seconds, if possible. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	void GetDuration(bool& bSuccess, double& Duration) const;


	/** Gets the resolution that this job will be rendered in. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FIntPoint GetOutputResolution() const;

	/** Gets the aspect ratio that this job will be rendered in. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	double GetOutputAspectRatio() const;

	/** Checks whether the job contains data that matches the search terms. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool MatchesSearchTerm(const FString& SearchTerm) const;

public:
	/** Returns the GUID, which is randomly generated at creation. */
	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FGuid GetGuid() const { return Guid; }

	/** Randomly generates a new GUID. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void GenerateNewGuid() { Guid = FGuid::NewGuid(); }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	int32 GetWaitFramesBeforeRendering() const { return WaitFramesBeforeRendering; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetWaitFramesBeforeRendering(const int32 NewWaitFramesBeforeRendering) { WaitFramesBeforeRendering = FMath::Max<int32>(0, NewWaitFramesBeforeRendering); }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	ULevelSequence* GetLevelSequence() const { return LevelSequence; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetLevelSequence(ULevelSequence* NewSequence) { LevelSequence = NewSequence; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetIsUsingCustomStartFrame() const { return bOverrideStartFrame; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetIsUsingCustomStartFrame(const bool bNewOverrideStartFrame) { bOverrideStartFrame = bNewOverrideStartFrame; }

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	int32 GetCustomStartFrame() const { return CustomStartFrame; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetCustomStartFrame(const int32 NewCustomStartFrame) { CustomStartFrame = NewCustomStartFrame; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetIsUsingCustomEndFrame() const { return bOverrideEndFrame; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetIsUsingCustomEndFrame(const bool bNewOverrideEndFrame) { bOverrideEndFrame = bNewOverrideEndFrame; }

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	int32 GetCustomEndFrame() const { return CustomEndFrame; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetCustomEndFrame(const int32 NewCustomEndFrame) { CustomEndFrame = NewCustomEndFrame; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetIsUsingCustomResolution() const { return bOverrideResolution; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetIsUsingCustomResolution(const bool bNewOverrideResolution) { bOverrideResolution = bNewOverrideResolution; }

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FIntPoint GetCustomResolution() const { return CustomResolution; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetCustomResolution(const FIntPoint NewCustomResolution) { CustomResolution = NewCustomResolution; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FString GetJobId() const { return JobId; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetJobId(const FString& NewJobId);
	void SetJobIdRaw(const FString& NewJobId) { JobId = NewJobId; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FString GetJobName() const { return JobName; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetJobName(const FString& NewJobName) { JobName = NewJobName; }
	void SetJobNameRaw(const FString& NewJobName) { JobName = NewJobName; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetIsEnabled() const { return bIsEnabled; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetIsEnabled(const bool bEnabled) { bIsEnabled = bEnabled; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FString GetOutputDirectory() const;
	FString GetOutputDirectoryRaw() const { return OutputDirectory; }

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	FString GetOutputDirectoryForDisplay() const { return OutputDirectory; }

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetOutputDirectory(const FString& NewOutputDirectory);
	void SetOutputDirectoryRaw(const FString& NewOutputDirectory) { OutputDirectory = NewOutputDirectory; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	UMoviePipelinePrimaryConfig* GetRenderPreset() const { return RenderPreset; }

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	UMoviePipelineOutputSetting* GetRenderPresetOutputSettings() const;

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	void SetRenderPreset(UMoviePipelinePrimaryConfig* NewRenderPreset) { RenderPreset = NewRenderPreset; }


	TArray<URemoteControlPreset*> GetRemoteControlPresets() const;

	bool HasStoredRemoteControlValueBytes(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity) const;
	bool GetRemoteControlValueBytes(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, TArray<uint8>& OutBytes) const;
	bool SetRemoteControlValueBytes(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes);

	bool HasStoredRemoteControlValueBytes(const FGuid& FieldId) const;
	bool GetRemoteControlValueBytes(const FGuid& FieldId, TArray<uint8>& OutBytes) const;
	bool SetRemoteControlValueBytes(const FGuid& FieldId, const TArray<uint8>& Bytes);

	TMap<FGuid, FRenderGridRemoteControlPropertyData>& GetRemoteControlValuesBytesRef() { return RemoteControlValues; }


	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetRemoteControlValue(const FGuid& FieldId, FString& Json) const;

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	bool SetRemoteControlValue(const FGuid& FieldId, const FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetRemoteControlFieldIdFromLabel(const FString& Label, FGuid& FieldId) const;

	UFUNCTION(BlueprintPure, Category="Render Grid Job")
	bool GetRemoteControlLabelFromFieldId(const FGuid& FieldId, FString& Label) const;

	UFUNCTION(BlueprintCallable, Category="Render Grid Job")
	TMap<FGuid, FString> GetRemoteControlValues() const;

private:
	/** The unique ID of this job. */
	UPROPERTY()
	FGuid Guid;

	/** Waits the given number of frames before it will render this job. This can be set to a higher amount when the renderer has to wait for your code to complete (such as construction scripts etc). Try increasing this value when rendering doesn't produce the output you expect it to. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", ClampMin="0"))
	int32 WaitFramesBeforeRendering;

	/** The level sequence, this is what will be rendered during rendering. A job without a level sequence can't be rendered. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true"))
	TObjectPtr<ULevelSequence> LevelSequence;

	/** If this is true, the CustomStartFrame property will override the start frame of the level sequence. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	bool bOverrideStartFrame;

	/** If bOverrideStartFrame is true, this property will override the start frame of the level sequence. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", EditCondition="bOverrideStartFrame"))
	int32 CustomStartFrame;

	/** If this is true, the CustomEndFrame property will override the end frame of the level sequence. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	bool bOverrideEndFrame;

	/** If bOverrideEndFrame is true, this property will override the end frame of the level sequence. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", EditCondition="bOverrideEndFrame"))
	int32 CustomEndFrame;

	/** If this is true, the CustomResolution property will override the resolution of the render. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", InlineEditConditionToggle))
	bool bOverrideResolution;

	/** If bOverrideResolution is true, this property will override the resolution of the render. */
	UPROPERTY(EditInstanceOnly, Category="Render Grid Job", Meta=(AllowPrivateAccess="true", EditCondition="bOverrideResolution", ClampMin="1", UIMin="1"))
	FIntPoint CustomResolution;

	/** If this is true, this job will be rendered during a batch rendering, otherwise it will be skipped. */
	UPROPERTY()
	bool bIsEnabled;

	/** The 'ID' of this job, this 'ID' is set by users. During rendering it will place all the output files of this job into a folder called after this 'ID', this means that this string can only contain file-safe characters. */
	UPROPERTY()
	FString JobId;

	/** The name of this job, this can be anything, it's set by the user, it serves as a way for the user to understand what job this is. */
	UPROPERTY()
	FString JobName;

	/** This is the folder in which the output files (of rendering) are placed into. To be more specific, the output files are placed in: {OutputDirectory}/{JobId}/, this folder will be created if it doesn't exist at the time of rendering. */
	UPROPERTY()
	FString OutputDirectory;

	/** The movie pipeline render preset. Render grid jobs are rendered using the movie pipeline plugin. This 'preset' contains the configuration of that plugin. */
	UPROPERTY()
	TObjectPtr<UMoviePipelinePrimaryConfig> RenderPreset;

	/** The remote control plugin can be used to customize and modify the way a job is rendered. If remote control is being used, the property values of this job will be stored in this map (remote control entity id -> value as bytes). */
	UPROPERTY()
	mutable TMap<FGuid, FRenderGridRemoteControlPropertyData> RemoteControlValues;
};


/**
 * This class represents a render grid, which is basically just a collection of render grid jobs.
 * A render grid is the asset that is shown in the content browser, it's the asset that can be opened and edited using the editor.
 */
UCLASS(Blueprintable, BlueprintType, EditInlineNew, Meta=(DontUseGenericSpawnObject="true"))
class RENDERGRID_API URenderGrid : public UObject
{
	GENERATED_BODY()

public:
	URenderGrid();

	/** Obtains a string representation of this object. Shouldn't be used for anything other than logging/debugging. */
	UFUNCTION(BlueprintPure, Category="Render Grid", Meta=(DisplayName="To Debug String (Render Grid)", CompactNodeTitle="DEBUG"))
	FString ToDebugString() const;

	/** Obtains a JSON representation of this object. Shouldn't be used for anything other than logging/debugging. */
	TSharedPtr<FJsonValue> ToDebugJson() const;

public:
	//UObject interface
	virtual UWorld* GetWorld() const override;
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	/** Copy jobs from the given RenderGrid to self. */
	void CopyJobs(URenderGrid* From);

	/** Copy all properties except jobs from the given RenderGrid to self. */
	void CopyAllPropertiesExceptJobs(URenderGrid* From);

	/** Copy all properties from the given RenderGrid to self. */
	void CopyAllProperties(URenderGrid* From);

	/** Copy all user variables (configured in the blueprint graph) from the given RenderGrid to self. */
	void CopyAllUserVariables(URenderGrid* From);

public:
	static TArray<FString> GetBlueprintImplementableEvents()
	{
		return {
			TEXT("ReceiveBeginEditor"),
			TEXT("ReceiveEndEditor"),
			TEXT("ReceiveTick"),
			TEXT("ReceiveBeginBatchRender"),
			TEXT("ReceiveEndBatchRender"),
			TEXT("ReceiveBeginJobRender"),
			TEXT("ReceiveEndJobRender"),
			TEXT("ReceiveBeginViewportRender"),
			TEXT("ReceiveEndViewportRender"),
		};
	}

protected:
	/**
	 * Event for when this asset is opened in the editor. The asset will also be closed and reopened during a blueprint compile.
	 * 
	 * In here, you could for example obtain jobs from an external source and replace the current jobs of this render grid asset with those.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="BeginEditor"))
	void ReceiveBeginEditor();

	/**
	 * Event for when this asset is closed in the editor. The asset will also be closed and reopened during a blueprint compile.
	 * 
	 * In here, you could do any cleanup required at the end of editing this render grid asset.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="EndEditor"))
	void ReceiveEndEditor();

	/**
	 * The tick event, will only execute when the asset is open in the editor.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="Tick"))
	void ReceiveTick(float DeltaTime);

	/**
	 * Event for when batch rendering begins.
	 * 
	 * In here, you could for example obtain jobs from an external source and add them to the queue.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="BeginBatchRender"))
	void ReceiveBeginBatchRender(URenderGridQueue* Queue);

	/**
	 * Event for when batch rendering ends.
	 * 
	 * In here, you could do any cleanup required at the end of a batch render.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="EndBatchRender"))
	void ReceiveEndBatchRender(URenderGridQueue* Queue);

	/**
	 * Event for when job rendering begins.
	 * 
	 * In here, you could for example change elements in the world according to what job this is.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="BeginJobRender"))
	void ReceiveBeginJobRender(URenderGridQueue* Queue, URenderGridJob* Job);

	/**
	 * Event for when job rendering ends.
	 * 
	 * In here, you could do any cleanup required at the end of rendering out a job,
	 * like for example undoing the changes you've made to the world for this job.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="EndJobRender"))
	void ReceiveEndJobRender(URenderGridQueue* Queue, URenderGridJob* Job);

	/**
	 * Event for when job rendering for the viewport viewer-mode begins.
	 * 
	 * This event will fire every frame, as long as the viewport viewer-mode is open.
	 * 
	 * In here, you could for example change elements in the world according to what job this is.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="BeginViewportRender"))
	void ReceiveBeginViewportRender(URenderGridJob* Job);

	/**
	 * Event for when job rendering for the viewport viewer-mode ends.
	 * 
	 * This event will fire every frame, as long as the viewport viewer-mode is open.
	 * 
	 * In here, you could do any cleanup required at the end of rendering out a job,
	 * like for example undoing the changes you've made to the world for this job.
	 */
	UFUNCTION(BlueprintImplementableEvent, Meta=(DisplayName="EndViewportRender"))
	void ReceiveEndViewportRender(URenderGridJob* Job);

public:
	/** Overridable native event for when this asset is opened in the editor. The asset will also be closed and reopened during a blueprint compile. */
	virtual void BeginEditor();

	/** Overridable native event for when this asset is closed in the editor. The asset will also be closed and reopened during a blueprint compile. */
	virtual void EndEditor();

	/** Overridable native event for when this asset is ticked by the editor. */
	virtual void Tick(float DeltaTime);

	/** Overridable native event for when batch rendering begins. */
	virtual void BeginBatchRender(URenderGridQueue* Queue);

	/** Overridable native event for when batch rendering ends. */
	virtual void EndBatchRender(URenderGridQueue* Queue);

	/** Overridable native event for when job rendering begins. */
	virtual void BeginJobRender(URenderGridQueue* Queue, URenderGridJob* Job);

	/** Overridable native event for when job rendering ends. */
	virtual void EndJobRender(URenderGridQueue* Queue, URenderGridJob* Job);

	/** Overridable native event for when job rendering for the viewport viewer-mode begins. */
	virtual void BeginViewportRender(URenderGridJob* Job);

	/** Overridable native event for when job rendering for the viewport viewer-mode ends. */
	virtual void EndViewportRender(URenderGridJob* Job);

private:
	DECLARE_DELEGATE_RetVal_TwoParams(URenderGridQueue*, FRenderGridRunRenderJobsDefaultObjectCallback, URenderGrid* /*DefaultObject*/, TArray<URenderGridJob*> /*Jobs*/);
	DECLARE_DELEGATE_RetVal_TwoParams(URenderGridQueue*, FRenderGridRunRenderJobsCallback, URenderGrid* /*This*/, TArray<URenderGridJob*> /*Jobs*/);
	URenderGridQueue* RunRenderJobs(const TArray<URenderGridJob*>& Jobs, const FRenderGridRunRenderJobsDefaultObjectCallback& DefaultObjectCallback, const FRenderGridRunRenderJobsCallback& Callback);

public:
	/** Renders all the enabled jobs of this render grid. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* Render();

	/** Renders the given job of this render grid. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderJob(URenderGridJob* Job);

	/** Renders all the given jobs of this render grid. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderJobs(const TArray<URenderGridJob*>& Jobs);


	/** Renders all the enabled jobs of this render grid. Only renders a single frame of each job. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderSingleFrame(const int32 Frame);

	/** Renders the given job of this render grid. Only renders a single frame. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderJobSingleFrame(URenderGridJob* Job, const int32 Frame);

	/** Renders all the given jobs of this render grid. Only renders a single frame of each job. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderJobsSingleFrame(const TArray<URenderGridJob*>& Jobs, const int32 Frame);


	/** Renders all the enabled jobs of this render grid. Only renders a single frame of each job. The frame number it renders is based on the given FramePosition (0.0 is the first frame, 1.0 is the last frame, 0.5 is the frame in the middle, etc). */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderSingleFramePosition(const double FramePosition);

	/** Renders the given job of this render grid. Only renders a single frame. The frame number it renders is based on the given FramePosition (0.0 is the first frame, 1.0 is the last frame, 0.5 is the frame in the middle, etc). */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderJobSingleFramePosition(URenderGridJob* Job, const double FramePosition);

	/** Renders all the given jobs of this render grid. Only renders a single frame of each job. The frame number it renders is based on the given FramePosition (0.0 is the first frame, 1.0 is the last frame, 0.5 is the frame in the middle, etc). */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridQueue* RenderJobsSingleFramePosition(const TArray<URenderGridJob*>& Jobs, const double FramePosition);

public:
	/** Returns the GUID, which is randomly generated at creation. */
	UFUNCTION(BlueprintPure, Category="Render Grid")
	FGuid GetGuid() const { return Guid; }

	/** Randomly generates a new GUID. */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void GenerateNewGuid() { Guid = FGuid::NewGuid(); }

public:
	URenderGridSettings* GetSettingsObject() { return Settings; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	TSoftObjectPtr<UWorld> GetLevel() const;

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void SetPropsSource(ERenderGridPropsSourceType InPropsSourceType, UObject* InPropsSourceOrigin = nullptr);

	UFUNCTION(BlueprintPure, Category="Render Grid")
	URenderGridPropsSourceBase* GetPropsSource() const;

	template<typename Type>
	Type* GetPropsSource() const
	{
		static_assert(TIsDerivedFrom<Type, URenderGridPropsSourceBase>::IsDerived, "Type needs to derive from URenderGridPropsSourceBase.");
		return Cast<Type>(GetPropsSource());
	}

	UFUNCTION(BlueprintPure, Category="Render Grid")
	ERenderGridPropsSourceType GetPropsSourceType() const { return Settings->PropsSourceType; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	UObject* GetPropsSourceOrigin() const;

public:
	URenderGridDefaults* GetDefaultsObject() { return Defaults; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	ULevelSequence* GetDefaultLevelSequence() const { return Defaults->LevelSequence; }

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void SetDefaultLevelSequence(ULevelSequence* NewSequence) { Defaults->LevelSequence = NewSequence; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	UMoviePipelinePrimaryConfig* GetDefaultRenderPreset() const { return Defaults->RenderPreset; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	UMoviePipelineOutputSetting* GetDefaultRenderPresetOutputSettings() const;

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void SetDefaultRenderPreset(UMoviePipelinePrimaryConfig* NewRenderPreset) { Defaults->RenderPreset = NewRenderPreset; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	FString GetDefaultOutputDirectory() const;
	FString GetDefaultOutputDirectoryRaw() const { return Defaults->OutputDirectory; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	FString GetDefaultOutputDirectoryForDisplay() const { return Defaults->OutputDirectory; }

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void SetDefaultOutputDirectory(const FString& NewOutputDirectory);
	void SetDefaultOutputDirectoryRaw(const FString& NewOutputDirectory) { Defaults->OutputDirectory = NewOutputDirectory; }

public:
	UFUNCTION(BlueprintCallable, Category="Render Grid", Meta=(Keywords="remove delete all"))
	void ClearRenderGridJobs();

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void SetRenderGridJobs(const TArray<URenderGridJob*>& Jobs);

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void AddRenderGridJob(URenderGridJob* Job);

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void RemoveRenderGridJob(URenderGridJob* Job);

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void InsertRenderGridJob(URenderGridJob* Job, int32 Index);

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	bool HasAnyRenderGridJobs() const { return !RenderGridJobs.IsEmpty(); }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	bool HasRenderGridJob(URenderGridJob* Job) const;

	UFUNCTION(BlueprintPure, Category="Render Grid")
	int32 GetIndexOfRenderGridJob(URenderGridJob* Job) const;

	TArray<TObjectPtr<URenderGridJob>>& GetRenderGridJobsRef() { return RenderGridJobs; }

	UFUNCTION(BlueprintPure, Category="Render Grid")
	TArray<URenderGridJob*> GetRenderGridJobs() const;

	UFUNCTION(BlueprintPure, Category="Render Grid")
	TArray<URenderGridJob*> GetEnabledRenderGridJobs() const;

	UFUNCTION(BlueprintPure, Category="Render Grid")
	TArray<URenderGridJob*> GetDisabledRenderGridJobs() const;

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void InsertRenderGridJobBefore(URenderGridJob* Job, URenderGridJob* BeforeJob);

	UFUNCTION(BlueprintCallable, Category="Render Grid")
	void InsertRenderGridJobAfter(URenderGridJob* Job, URenderGridJob* AfterJob);

public:
	/** Generates a unique job ID by grabbing the current time, as well as 16 random bytes, and converting that to a base64 string. **/
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	FString GenerateUniqueRandomJobId();

	/** Generates a unique job ID by finding the currently highest job ID and increasing it by one. **/
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	FString GenerateNextJobId();

	/** Finds whether given job ID already exists in this grid. **/
	UFUNCTION(BlueprintPure, Category="Render Grid")
	bool DoesJobIdExist(const FString& JobId);

	/** Creates a new job. This job won't be added to the grid, so it will eventually be garbage collected. **/
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridJob* CreateTempRenderGridJob();

	/** Creates a new job and adds it to this grid. **/
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridJob* CreateAndAddNewRenderGridJob();

	/** Copy the given job in this grid. **/
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	URenderGridJob* DuplicateAndAddRenderGridJob(URenderGridJob* Job);

	/** Relocates the given job in this grid to the position of the given dropped-on job. **/
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	bool ReorderRenderGridJob(URenderGridJob* Job, URenderGridJob* DroppedOnJob, const bool bAfter = true);

public:
	/** Returns true when it's currently executing a blueprint implementable event, returns false otherwise. */
	bool IsCurrentlyExecutingUserCode() const { return bExecutingBlueprintEvent; }

private:
	/** The unique ID of this render grid. */
	UPROPERTY()
	FGuid Guid;


	/** The settings of this render grid. */
	UPROPERTY(Instanced)
	TObjectPtr<URenderGridSettings> Settings;

	/** The default values for new jobs. */
	UPROPERTY(Instanced)
	TObjectPtr<URenderGridDefaults> Defaults;

	/** The jobs of this render grid. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<URenderGridJob>> RenderGridJobs;


	/** True when it's currently executing a blueprint event, false otherwise. */
	UPROPERTY(Transient)
	bool bExecutingBlueprintEvent;


	/** GetWorld calls can be expensive, we speed them up by caching the last found world until it goes away. */
	mutable TWeakObjectPtr<UWorld> CachedWorldWeakPtr;
};
