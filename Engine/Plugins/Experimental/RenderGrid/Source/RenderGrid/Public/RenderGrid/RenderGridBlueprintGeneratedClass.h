// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "RenderGridBlueprintGeneratedClass.generated.h"


/**
 * A UBlueprintGeneratedClass child class for the RenderGrid modules.
 *
 * Required in order for a RenderGrid to be able to have a blueprint graph.
 */
UCLASS()
class RENDERGRID_API URenderGridBlueprintGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()
public:
	//~ Begin UBlueprintGeneratedClass interface
	virtual uint8* GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const override;
	//~ End UBlueprintGeneratedClass interface
};
