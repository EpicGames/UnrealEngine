// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/StrongObjectPtr.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGridManager.generated.h"


class URenderGridJob;
class URenderGrid;
class URenderGridPropRemoteControl;
class URenderGridQueue;
class UTexture2D;


/**
 * This struct keeps track of the values of the properties before new values were applied, so we can rollback to the previous state.
 */
USTRUCT()
struct RENDERGRID_API FRenderGridManagerPreviousPropValues
{
	GENERATED_BODY()

public:
	FRenderGridManagerPreviousPropValues() = default;
	FRenderGridManagerPreviousPropValues(const TMap<TObjectPtr<URenderGridPropRemoteControl>, FRenderGridRemoteControlPropertyData>& InRemoteControlData)
		: RemoteControlData(InRemoteControlData)
	{}

public:
	bool IsEmpty() const { return RemoteControlData.IsEmpty(); }

public:
	/** The previous values of the remote control properties. */
	UPROPERTY()
	TMap<TObjectPtr<URenderGridPropRemoteControl>, FRenderGridRemoteControlPropertyData> RemoteControlData;
};


namespace UE::RenderGrid
{
	/** A delegate for when FRenderGridManager::RenderPreviewFrame has finished. */
	DECLARE_DELEGATE_OneParam(FRenderGridManagerRenderPreviewFrameArgsCallback, bool /*bSuccess*/);


	/**
	 * The arguments for the FRenderGridManager::RenderPreviewFrame function.
	 */
	struct RENDERGRID_API FRenderGridManagerRenderPreviewFrameArgs
	{
	public:
		/** Whether it should run invisibly (so without any UI elements popping up during rendering) or not. */
		bool bHeadless = false;

		/** The render grid of the given render grid jobs that will be rendered. */
		TStrongObjectPtr<URenderGrid> RenderGrid = nullptr;

		/** The specific render grid job that will be rendered. */
		TStrongObjectPtr<URenderGridJob> RenderGridJob = nullptr;

		/** The specific frame number that will be rendered. */
		TOptional<int32> Frame;

		/** The resolution it will be rendered in. */
		FIntPoint Resolution = FIntPoint(0, 0);

		/** The delegate for when the rendering has finished. */
		FRenderGridManagerRenderPreviewFrameArgsCallback Callback;
	};


	/**
	 * The singleton class that manages render grids.
	 * 
	 * This functionality is separated from the UI in order to make it is reusable, meaning that it can also be used in other modules.
	 */
	class RENDERGRID_API FRenderGridManager
	{
	public:
		/** A folder in which rendered frames for temporary use will be placed in. */
		inline static FString TmpRenderedFramesPath = FPaths::AutomationTransientDir() / TEXT("RenderGrid");

		/** The number of characters for a generated ID. For example, a value of 4 results in IDs: "0001", "0002", etc. */
		static constexpr int32 GeneratedIdCharacterLength = 4;

	public:
		/** Batch render the given render grid job(s) of the given render grid. */
		URenderGridQueue* CreateBatchRenderQueue(URenderGrid* Grid, const TArray<URenderGridJob*>& Jobs);

		/** Batch render the given render grid job(s) of the given render grid. Only renders a single frame of each job. */
		URenderGridQueue* CreateBatchRenderQueueSingleFrame(URenderGrid* Grid, const TArray<URenderGridJob*>& Jobs, const int32 Frame);

		/** Batch render the given render grid job(s) of the given render grid. Only renders a single frame of each job. The frame number it renders is based on the given FramePosition (0.0 is the first frame, 1.0 is the last frame, 0.5 is the frame in the middle, etc). */
		URenderGridQueue* CreateBatchRenderQueueSingleFramePosition(URenderGrid* Grid, const TArray<URenderGridJob*>& Jobs, const double FramePosition);


		/** Render a preview frame (or multiple if no frame number is specified) of the given render grid job. */
		URenderGridQueue* RenderPreviewFrame(const FRenderGridManagerRenderPreviewFrameArgs& Args);

		/** Gets the rendered preview frame (of a rendering in which the frame number was specified). */
		UTexture2D* GetSingleRenderedPreviewFrame(URenderGridJob* Job, UTexture2D* ReusingTexture2D, bool& bOutReusedGivenTexture2D);

		/** Gets the rendered preview frame (of a rendering in which the frame number was specified). */
		UTexture2D* GetSingleRenderedPreviewFrame(URenderGridJob* Job);

		/** Gets the rendered preview frame of the given frame number (of a rendering in which the frame number was not specified). */
		UTexture2D* GetRenderedPreviewFrame(URenderGridJob* Job, const int32 Frame, UTexture2D* ReusingTexture2D, bool& bOutReusedGivenTexture2D);

		/** Gets the rendered preview frame of the given frame number (of a rendering in which the frame number was not specified). */
		UTexture2D* GetRenderedPreviewFrame(URenderGridJob* Job, const int32 Frame);


		/** Makes sure that all the data from the current props source is stored in all of the render grid jobs of this render grid. */
		void UpdateStoredRenderGridJobsPropValues(URenderGrid* Grid);

		/** Applies the props of the given render grid job, also requires the render grid to be given as well (to know what props the render grid job is using). */
		FRenderGridManagerPreviousPropValues ApplyJobPropValues(const URenderGrid* Grid, const URenderGridJob* Job);

		/** Restores the props that were previously applied, to the values that they were before. */
		void RestorePropValues(const FRenderGridManagerPreviousPropValues& PreviousPropValues);

	private:
		/** The map that stores the start frame (of a render) of each rendered render grid job. */
		TMap<FGuid, int32> StartFrameOfRenders;
	};
}
