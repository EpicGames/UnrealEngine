// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RemoteControlPreset.h"
#include "RenderGridPropsSource.generated.h"


/////////////////////////////////////////////////////
// Enums

/**
 * The type of the properties source.
 * In other words, where the properties come from that each render grid job can have.
 */
UENUM(BlueprintType)
enum class ERenderGridPropsSourceType : uint8
{
	Local = 0 UMETA(Hidden, DisplayName = "Local Source"),
	RemoteControl = 1 UMETA(DisplayName = "Remote Control Preset")
};


/////////////////////////////////////////////////////
// Base classes

/**
 * The base class of the render grid property abstraction.
 */
UCLASS(Abstract, HideCategories=Object)
class RENDERGRID_API URenderGridPropBase : public UObject
{
	GENERATED_BODY()
};

/**
 * The base class of the render grid properties abstraction.
 */
UCLASS(Abstract, HideCategories=Object)
class RENDERGRID_API URenderGridPropsBase : public UObject
{
	GENERATED_BODY()

public:
	/** Returns all props. */
	virtual TArray<URenderGridPropBase*> GetAll() const { return TArray<URenderGridPropBase*>(); }
};

/**
 * The base class of the render grid properties source abstraction.
 */
UCLASS(Abstract, BlueprintType, HideCategories=Object)
class RENDERGRID_API URenderGridPropsSourceBase : public UObject
{
	GENERATED_BODY()

public:
	/** Returns the GUID, which is randomly generated at creation. */
	FGuid GetGuid() const { return Guid; }

	/** Randomly generates a new GUID. */
	void GenerateNewGuid() { Guid = FGuid::NewGuid(); }


	/** Returns the type of this properties source. */
	virtual ERenderGridPropsSourceType GetType() const { return ERenderGridPropsSourceType::Local; }

	/** Sets the properties source. */
	virtual void SetSourceOrigin(UObject* SourceOrigin) {}

	/** Returns the collection of properties (that this properties source contains). */
	virtual URenderGridPropsBase* GetProps() const { return NewObject<URenderGridPropsBase>(const_cast<URenderGridPropsSourceBase*>(this)); }

protected:
	UPROPERTY()
	FGuid Guid = FGuid::NewGuid();
};


/////////////////////////////////////////////////////
// Local source classes

/**
 * The local properties implementation of the render grid property abstraction.
 */
UCLASS(HideCategories=Object)
class RENDERGRID_API URenderGridPropLocal : public URenderGridPropBase
{
	GENERATED_BODY()
};

/**
 * The local properties implementation of the render grid properties abstraction.
 */
UCLASS(HideCategories=Object)
class RENDERGRID_API URenderGridPropsLocal : public URenderGridPropsBase
{
	GENERATED_BODY()
};

/**
 * The local properties implementation of the render grid properties source abstraction.
 */
UCLASS(BlueprintType, HideCategories=Object, Meta=(DontUseGenericSpawnObject="true"))
class RENDERGRID_API URenderGridPropsSourceLocal : public URenderGridPropsSourceBase
{
	GENERATED_BODY()

public:
	//~ Begin URenderGridPropsSourceBase interface
	virtual ERenderGridPropsSourceType GetType() const override { return ERenderGridPropsSourceType::Local; }
	virtual URenderGridPropsBase* GetProps() const override { return NewObject<URenderGridPropsLocal>(const_cast<URenderGridPropsSourceLocal*>(this)); }
	//~ End URenderGridPropsSourceBase interface
};


/////////////////////////////////////////////////////
// Remote control source classes

/**
 * The remote control properties implementation of the render grid property abstraction.
 */
UCLASS(HideCategories=Object)
class RENDERGRID_API URenderGridPropRemoteControl : public URenderGridPropBase
{
	GENERATED_BODY()

public:
	/** Gets the value (as bytes) of the given property (remote control entity). Returns true if the operation was successful, false otherwise. */
	static bool GetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, TArray<uint8>& OutBytes);

	/** Set the value (as bytes) of the given property (remote control entity). Returns true if the operation was successful, false otherwise. */
	static bool SetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes);

	/** Tests if it can set the value (as bytes) of the given property (remote control entity). Returns true if the set operation would likely be successful, false otherwise. */
	static bool CanSetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes);

public:
	/** Sets the initial values of this instance. */
	void Initialize(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity);

	/** Returns the property, which is a remote control entity (which can be a field or a function). */
	TSharedPtr<FRemoteControlEntity> GetRemoteControlEntity() const { return RemoteControlEntity; }

	/** Gets the value (as bytes) of this property. Returns true if the operation was successful, false otherwise. */
	bool GetValue(TArray<uint8>& OutBytes) const { return GetValueOfEntity(RemoteControlEntity, OutBytes); }

	/** Set the value (as bytes) of this property. Returns true if the operation was successful, false otherwise. */
	bool SetValue(const TArray<uint8>& Bytes) { return SetValueOfEntity(RemoteControlEntity, Bytes); }

	/** Tests if it can set the value (as bytes) of this property. Returns true if the set operation would likely be successful, false otherwise. */
	bool CanSetValue(const TArray<uint8>& Bytes) { return CanSetValueOfEntity(RemoteControlEntity, Bytes); }

protected:
	/** The property, which is a remote control entity (which can be a field or a function). */
	TSharedPtr<FRemoteControlEntity> RemoteControlEntity = nullptr;
};

/**
 * The remote control properties implementation of the render grid properties abstraction.
 */
UCLASS(HideCategories=Object)
class RENDERGRID_API URenderGridPropsRemoteControl : public URenderGridPropsBase
{
	GENERATED_BODY()

public:
	/** Sets the initial values of this instance. */
	void Initialize(URemoteControlPreset* InRemoteControlPreset);

	//~ Begin URenderGridPropsBase interface
	virtual TArray<URenderGridPropBase*> GetAll() const override;
	//~ End URenderGridPropsBase interface

	/** Returns all props, casted to URenderGridPropRemoteControl, for ease of use. */
	TArray<URenderGridPropRemoteControl*> GetAllCasted() const;

	/** Returns the prop with the given ID, or NULL if not found. */
	URenderGridPropRemoteControl* GetById(const FGuid& Id) const;

	/** Returns the prop with the given label, or NULL if not found. */
	URenderGridPropRemoteControl* GetByLabel(const FString& Label) const;

	/** Returns the remote control preset. */
	URemoteControlPreset* GetRemoteControlPreset() const { return (IsValid(RemoteControlPreset) ? RemoteControlPreset : nullptr); }

protected:
	/** The source of properties, which is a remote control preset. */
	UPROPERTY()
	TObjectPtr<URemoteControlPreset> RemoteControlPreset = nullptr;
};

/**
 * The remote control properties implementation of the render grid properties source abstraction.
 */
UCLASS(BlueprintType, HideCategories=Object, Meta=(DontUseGenericSpawnObject="true"))
class RENDERGRID_API URenderGridPropsSourceRemoteControl : public URenderGridPropsSourceBase
{
	GENERATED_BODY()

public:
	//~ Begin URenderGridPropsSourceBase interface
	virtual ERenderGridPropsSourceType GetType() const override { return ERenderGridPropsSourceType::RemoteControl; }
	virtual void SetSourceOrigin(UObject* SourceOrigin) override;
	virtual URenderGridPropsRemoteControl* GetProps() const override;
	//~ End URenderGridPropsSourceBase interface

	/** Returns (in the out parameter) the preset groups that are available in this remote control preset. */
	void GetAvailablePresetGroups(TArray<FName>& OutPresetGroups) const;

	/** Returns the remote control preset. */
	URemoteControlPreset* GetRemoteControlPreset() const { return (IsValid(RemoteControlPreset) ? RemoteControlPreset : nullptr); }

protected:
	/** The source of properties, which is a remote control preset. */
	UPROPERTY(VisibleInstanceOnly, Category="Render Grid|Remote Control")
	TObjectPtr<URemoteControlPreset> RemoteControlPreset = nullptr;

	/** The preset group (of the remote control preset) that we should obtain the properties from. */
	UPROPERTY(VisibleInstanceOnly, Category="Render Grid|Remote Control")
	FName ActivePresetGroup;
};
