// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGridManager.h"
#include "Utils/RenderGridUtils.h"
#include "Tickable.h"
#include "UObject/ObjectPtr.h"
#include "RenderGridQueue.generated.h"


class FJsonValue;
class UMoviePipelineExecutorBase;
class UMoviePipelineExecutorJob;
class UMoviePipelineOutputBase;
class UMoviePipelinePIEExecutor;
class UMoviePipelineQueue;
class UMoviePipelineSetting;
class URenderGrid;
class URenderGridJob;
class URenderGridQueue;
struct FMoviePipelineOutputData;

namespace UE::RenderGrid::Private
{
	class FRenderGridGenericExecutionQueue;
}


namespace UE::RenderGrid
{
	/** A delegate for when a rendering queue is about to start. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRenderGridQueueStarted, URenderGridQueue* /*Queue*/);

	/** A delegate for when a rendering queue has finished. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRenderGridQueueFinished, URenderGridQueue* /*Queue*/, bool /*bSuccess*/);

	/**
	 * The arguments for the URenderGridQueue::Create function.
	 */
	struct RENDERGRID_API FRenderGridQueueCreateArgs
	{
	public:
		/** The render grid of the given render grid jobs that will be rendered. */
		TStrongObjectPtr<URenderGrid> RenderGrid = nullptr;

		/** The specific render grid jobs that will be rendered. */
		TArray<TStrongObjectPtr<URenderGridJob>> RenderGridJobs;

		/** If not null, it will override the movie pipeline executor class with this class. */
		TSubclassOf<UMoviePipelinePIEExecutor> PipelineExecutorClass = nullptr;

		/** The movie pipeline settings classes to disable (things like Anti-Aliasing, High-Res, etc). */
		TArray<TSubclassOf<UMoviePipelineSetting>> DisablePipelineSettingsClasses;

		/** The specific frame number that will be rendered. */
		TOptional<int32> Frame;

		/** The specific frame number (percentage-wise, between 0.0 and 1.0) that will be rendered. */
		TOptional<double> FramePosition;

		/** Whether it should run the Begin and End Batch Render events or not. */
		bool bIsBatchRender = false;

		/** Whether it should run invisibly (so without any UI elements popping up during rendering) or not. */
		bool bHeadless = false;

		/** Whether it should make sure it will output an image or not (if this bool is true, it will test if JPG/PNG/etc output is enabled, if none are, it will enable PNG output). */
		bool bForceOutputImage = false;

		/** Whether it should make sure it will only output in a single format (if this bool is true, if for example JPG and PNG output are enabled, one will be disabled, so that there will only be 1 output that's enabled). */
		bool bForceOnlySingleOutput = false;

		/** Whether it should use the sequence's framerate rather than any manually set framerate (if this bool is true, it will make sure bUseCustomFrameRate is set to false). */
		bool bForceUseSequenceFrameRate = false;

		/** Whether it should make sure it will output files named 0000000001, 0000000002, etc (if this bool is true, it will override the FileNameFormat to simply output the frame number, and it will add 1000000000 to that frame number to hopefully ensure that any negative frame numbers will not result in filenames starting with a minus character). */
		bool bEnsureSequentialFilenames = false;
	};
}


/**
 * This class is responsible for the movie pipeline part of the rendering of the given render grid job.
 */
UCLASS()
class RENDERGRID_API URenderGridMoviePipelineRenderJob : public UObject
{
	GENERATED_BODY()

public:
	/** Creates a new render job, it won't be started right away. */
	static URenderGridMoviePipelineRenderJob* Create(URenderGridQueue* Queue, URenderGridJob* Job, const UE::RenderGrid::FRenderGridQueueCreateArgs& Args);

	/** The destructor, cleans up the TPromise (if it's set). */
	virtual void BeginDestroy() override;

	/** Starts this render job. */
	TSharedFuture<void> Execute();

	/** Cancels this render job. Relies on the internal movie pipeline implementation of job canceling on whether this will do anything or not. */
	void Cancel();

	/** Retrieves the rendering status of the given render grid job. */
	FString GetStatus() const;

	/** Retrieves the rendering status of the given render grid job. */
	float GetStatusPercentage() const;

	/** Retrieves the "Engine Warm Up Count" value from the AntiAliasingSettings from the render preset that this render grid job uses. */
	int32 GetEngineWarmUpCount() const;

public:
	/** An event that will be fired when the rendering process is waiting for this job's engine warm up. */
	void OnWaitForEngineWarmUpCount();

	/** An event that will be fired when the rendering process is executing the BeginJobRender event for this job. */
	void OnPreRenderEvent();

	/** An event that will be fired when the rendering process is executing the EndJobRender event for this job. */
	void OnPostRenderEvent();

public:
	/** Returns true if this render job was canceled (which for example can be caused by calling Cancel(), or by closing the render popup). */
	bool IsCanceled() const { return bCanceled; }

	/** Returns true if this render job can render (otherwise it will be skipped). */
	bool CanExecute() const { return bCanExecute; }

private:
	void ComputePlaybackContext(bool& bOutAllowBinding);
	void ExecuteFinished(UMoviePipelineExecutorBase* InPipelineExecutor, const bool bSuccess);

protected:
	/** The render grid job that will be rendered. */
	UPROPERTY(Transient)
	TObjectPtr<URenderGridJob> RenderGridJob;

	/** The render grid that the render grid job (that will be rendered) belongs to. */
	UPROPERTY(Transient)
	TObjectPtr<URenderGrid> RenderGrid;

	/** The movie pipeline queue. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineQueue> PipelineQueue;

	/** The movie pipeline executor. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorBase> PipelineExecutor;

	/** The movie pipeline executor job of the given render grid job. */
	UPROPERTY(Transient)
	TObjectPtr<UMoviePipelineExecutorJob> PipelineExecutorJob;

	/** The TPromise of the rendering process. */
	TSharedPtr<TPromise<void>> Promise;

	/** The TFuture of the rendering process. */
	TSharedFuture<void> PromiseFuture;

	/** The rendering status of the given render grid job. */
	UPROPERTY(Transient)
	FString Status;

	/** Whether the entry can execute, or whether it should just skip execution. */
	UPROPERTY(Transient)
	bool bCanExecute;

	/** Whether the entry was canceled (like by calling Cancel(), or by closing the render popup). */
	UPROPERTY(Transient)
	bool bCanceled;

	/** Whether the entry has finished (or was canceled). */
	UPROPERTY(Transient)
	bool bFinished;
};


/**
 * This class is responsible for rendering the given render grid jobs.
 */
UCLASS(BlueprintType, Meta=(DontUseGenericSpawnObject="true"))
class RENDERGRID_API URenderGridQueue : public UObject, public FTickableGameObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnRenderingQueueChanged);
	/** The event that will be fired when the currently rendering queue changes (like for example when the previous one has finished rendering). */
	static FOnRenderingQueueChanged& OnRenderingQueueChanged() { return OnRenderingQueueChangedDelegate; }

	/** Call this function to make it so that the editor will be closed when all rendering queues have finished. This function has to only be called once. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="execution execute finish close stop end done quit"))
	static void CloseEditorOnCompletion();

private:
	/** Closes the editor, but only if CloseEditorOnCompletion has been called. */
	static void RequestAppExitIfSetToExitOnCompletion();

public:
	/** Returns true if it's currently rendering any queues. */
	static bool IsRenderingAny();

	/** Returns the currently rendering queue, or NULL if there isn't any currently rendering. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="executing execution execute running rendering"))
	static URenderGridQueue* GetCurrentlyRenderingQueue();

	/** Returns the number of rendering queues that are currently queued up. This also includes the currently rendering queue. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="executing execution execute running rendering left todo queued"))
	static int32 GetRemainingRenderingQueuesCount();

private:
	/** Adds the given queue. Will automatically start rendering it if it's the only one currently. */
	static void AddRenderingQueue(URenderGridQueue* Queue);

	/** Starts the next rendering queue, if there is one. Nothing will happen until the next tick. */
	static void DoNextRenderingQueue();

private:
	/** The queue of rendering queues. This contains the currently rendering queue, as well as the ones that should be rendered after it. */
	static TArray<TObjectPtr<URenderGridQueue>> RenderingQueues;

	/** The delegate for when data in the ExecutingQueues has changed. */
	static FOnRenderingQueueChanged OnRenderingQueueChangedDelegate;

	/** Whether the editor should be closed when all rendering queues finish rendering. */
	static bool bExitOnCompletion;

public:
	//~ Begin FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderGridQueue, STATGROUP_Tickables);
	}
	//~ End FTickableGameObject interface

public:
	/** Creates a new render queue, it won't be started right away. */
	static URenderGridQueue* Create(const UE::RenderGrid::FRenderGridQueueCreateArgs& Args);

	/** Obtains a string representation of this object. Shouldn't be used for anything other than logging/debugging. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(DisplayName="To Debug String (Render Grid Queue)", CompactNodeTitle="DEBUG"))
	FString ToDebugString() const;

	/** Obtains a JSON representation of this object. Shouldn't be used for anything other than logging/debugging. */
	TSharedPtr<FJsonValue> ToDebugJson() const;

	/** Starts this render queue. */
	void Execute();

public:
	/** Returns the GUID, which is randomly generated at creation. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue")
	FGuid GetGuid() const { return Guid; }

	/** Randomly generates a new GUID. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue")
	void GenerateNewGuid() { Guid = FGuid::NewGuid(); }

	/** Queues the given job. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="render append"))
	void AddJob(URenderGridJob* Job);

	/** Pauses the queue. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="wait"))
	void Pause();

	/** Resumes the queue. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="unwait unpause"))
	void Resume();

	/** Cancels the current and the remaining queued jobs. Relies on the internal movie pipeline implementation of job canceling on whether this will stop the current render grid job from rendering or not. Will always prevent new render grid jobs from rendering. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="stop quit exit kill terminate end"))
	void Cancel();

	/** Returns true if this queue has been started. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="stopped quited exited killed terminated ended succeeded completed finished canceled cancelled"))
	bool IsStarted() const { return bStarted; }

	/** Returns true if this queue is currently paused. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="stopped quited exited killed terminated ended succeeded completed finished canceled cancelled"))
	bool IsPaused() const { return (bPaused && !bCanceled && !bFinished); }

	/** Returns true if this queue has been canceled. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="stopped quited exited killed terminated ended succeeded completed finished canceled cancelled"))
	bool IsCanceled() const { return bCanceled; }

	/** Returns true if this queue has been canceled. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="stopped quited exited killed terminated ended succeeded completed finished canceled cancelled"))
	bool IsFinished() const { return bFinished; }

	/** Returns true if this queue is the one that's currently rendering, returns false if it hasn't started yet, or if it's waiting in the queue, or if it has finished. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="stopped quited exited killed terminated ended succeeded completed finished canceled cancelled"))
	bool IsCurrentlyRendering() const { return (GetCurrentlyRenderingQueue() == this); }

	/** Retrieves the rendering status of the given render grid job. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering"))
	URenderGrid* GetRenderGrid() const { return RenderGrid; }

	/** Retrieves the currently rendering render grid job, can return NULL. */
	UFUNCTION(BlueprintCallable, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain"))
	URenderGridJob* GetCurrentlyRenderingJob() const;

	/** Retrieves the rendering status of the given render grid job. Will return an empty string if this job wasn't found in this queue. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain text"))
	FString GetJobStatus(URenderGridJob* Job) const;

	/** Returns the percentage of the rendering status of the given render grid job. Will return 0 if this job wasn't found in this queue. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain number amount finished complete completion"))
	float GetJobStatusPercentage(URenderGridJob* Job) const;

	/** Returns all the jobs that have been and will be rendered. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain"))
	TArray<URenderGridJob*> GetJobs() const;

	/** Returns the number of jobs that have been and will be rendered. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain number amount"))
	int32 GetJobsCount() const;

	/** Returns the number of jobs that are still left to render, includes the job that is currently rendering. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain number amount finished"))
	int32 GetJobsRemainingCount() const;

	/** Returns the number of jobs that have finished rendering. Basically just returns [Get Jobs Count] minus [Get Jobs Remaining Count]. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain number amount finished"))
	int32 GetJobsCompletedCount() const;

	/** Returns the status of the rendering process. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain text"))
	FString GetStatus() const;

	/** Returns the percentage of jobs finished, this includes the progression of the job that is currently rendering. */
	UFUNCTION(BlueprintPure, Category="Render Grid Queue", Meta=(Keywords="rendering progression obtain number amount finished complete completion"))
	float GetStatusPercentage() const;

protected:
	void OnStart();
	void OnProcessJob(URenderGridJob* Job);
	void OnFinish();

protected:
	/** The unique ID of this queue. */
	UPROPERTY()
	FGuid Guid;

	/** The queue containing the render actions. */
	UE::RenderGrid::FRenderGridQueueCreateArgs Args;

	/** The queue containing the render actions. */
	TSharedPtr<UE::RenderGrid::Private::FRenderGridGenericExecutionQueue> Queue;

	/** The render grid jobs that are to be rendered, removed when one is grabbed from it and added to the execution queue. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<URenderGridJob>> RemainingJobs;

	/** The render grid jobs that are to be rendered, mapped to the movie pipeline render job of each specific render grid job. */
	UPROPERTY(Transient)
	TMap<TObjectPtr<URenderGridJob>, TObjectPtr<URenderGridMoviePipelineRenderJob>> Entries;

	/** The render grid jobs that is currently being rendered. */
	UPROPERTY(Transient)
	TObjectPtr<URenderGridJob> CurrentJob;

	/** The movie pipeline render job that is currently being rendered. */
	UPROPERTY(Transient)
	TObjectPtr<URenderGridMoviePipelineRenderJob> CurrentEntry;

	/** The render grid of the given render grid job that will be rendered. */
	UPROPERTY(Transient)
	TObjectPtr<URenderGrid> RenderGrid;

	/** Whether the queue has been started yet. */
	UPROPERTY(Transient)
	bool bStarted;

	/** Whether the queue has been paused. */
	UPROPERTY(Transient)
	bool bPaused;

	/** Whether the queue has been canceled. */
	UPROPERTY(Transient)
	bool bCanceled;

	/** Whether the queue has been finished. */
	UPROPERTY(Transient)
	bool bFinished;

	/** The property values that have been overwritten by the currently applied render grid job property values. */
	UPROPERTY(Transient)
	FRenderGridManagerPreviousPropValues PreviousProps;

	/** The engine framerate settings values that have been overwritten by the currently applied engine framerate settings values. */
	UPROPERTY(Transient)
	FRenderGridPreviousEngineFpsSettings PreviousFrameLimitSettings;

public:
	/** A delegate for when the queue is about to start. */
	UE::RenderGrid::FOnRenderGridQueueStarted& OnExecuteStarted() { return OnExecuteStartedDelegate; }

	/** A delegate for when the queue has finished. */
	UE::RenderGrid::FOnRenderGridQueueFinished& OnExecuteFinished() { return OnExecuteFinishedDelegate; }

private:
	UE::RenderGrid::FOnRenderGridQueueStarted OnExecuteStartedDelegate;
	UE::RenderGrid::FOnRenderGridQueueFinished OnExecuteFinishedDelegate;
};
