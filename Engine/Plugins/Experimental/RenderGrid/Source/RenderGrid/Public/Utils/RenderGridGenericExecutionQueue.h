// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "Containers/List.h"
#include "Tickable.h"


namespace UE::RenderGrid::Private
{
	/**
	 * Struct containing the delay data (such as the frames or the seconds of delay this delay requires before it can move on to the next step).
	 */
	struct RENDERGRID_API FRenderGridGenericExecutionQueueDelay
	{
	public:
		static FRenderGridGenericExecutionQueueDelay Frames(const int64 Frames) { return FRenderGridGenericExecutionQueueDelay(Frames); }
		static FRenderGridGenericExecutionQueueDelay Seconds(const double Seconds) { return FRenderGridGenericExecutionQueueDelay(Seconds); }
		static FRenderGridGenericExecutionQueueDelay FramesOrSeconds(const int64 Frames, const double Seconds) { return FRenderGridGenericExecutionQueueDelay(Frames, Seconds); }

	public:
		FRenderGridGenericExecutionQueueDelay(TYPE_OF_NULLPTR)
		{}

	private:
		FRenderGridGenericExecutionQueueDelay()
		{}

		explicit FRenderGridGenericExecutionQueueDelay(const int64 Frames)
			: MinimumFrames(Frames)
		{}

		explicit FRenderGridGenericExecutionQueueDelay(const double Seconds)
			: MinimumSeconds(Seconds)
		{}

		explicit FRenderGridGenericExecutionQueueDelay(const int64 Frames, const double Seconds)
			: MinimumFrames(Frames)
			, MinimumSeconds(Seconds)
		{}

	public:
		const int64 MinimumFrames = 0;
		const double MinimumSeconds = 0.0f;
	};


	/** A delegate for a queued action. */
	DECLARE_DELEGATE(FRenderGridGenericExecutionQueueAction);

	/** A delegate for a queued action, that optionally requires a delay after its execution. */
	DECLARE_DELEGATE_RetVal(FRenderGridGenericExecutionQueueDelay, FRenderGridGenericExecutionQueueActionReturningDelay);

	/** A delegate for a queued action, that will delay execution until the returned TFuture finishes. */
	DECLARE_DELEGATE_RetVal(TSharedFuture<void>, FRenderGridGenericExecutionQueueActionReturningDelayFuture);

	/** A delegate for a queued action, that will delay execution until the returned TFuture finishes, which can optionally return yet another delay if required. */
	DECLARE_DELEGATE_RetVal(TSharedFuture<FRenderGridGenericExecutionQueueDelay>, FRenderGridGenericExecutionQueueActionReturningDelayFutureReturningDelay);

	/**
	 * Struct containing the data of a queued action.
	 */
	struct RENDERGRID_API FRenderGridGenericExecutionQueueEntry
	{
	public:
		FRenderGridGenericExecutionQueueEntry(const FRenderGridGenericExecutionQueueAction& Action)
			: ActionRegular(Action)
		{}

		FRenderGridGenericExecutionQueueEntry(const FRenderGridGenericExecutionQueueActionReturningDelay& Action)
			: ActionReturningDelay(Action)
		{}

		FRenderGridGenericExecutionQueueEntry(const FRenderGridGenericExecutionQueueActionReturningDelayFuture& Action)
			: ActionReturningDelayFuture(Action)
		{}

		FRenderGridGenericExecutionQueueEntry(const FRenderGridGenericExecutionQueueActionReturningDelayFutureReturningDelay& Action)
			: ActionReturningDelayFutureReturningDelay(Action)
		{}

	public:
		const FRenderGridGenericExecutionQueueAction ActionRegular;
		const FRenderGridGenericExecutionQueueActionReturningDelay ActionReturningDelay;
		const FRenderGridGenericExecutionQueueActionReturningDelayFuture ActionReturningDelayFuture;
		const FRenderGridGenericExecutionQueueActionReturningDelayFutureReturningDelay ActionReturningDelayFutureReturningDelay;
	};


	/**
	 * This class provides generic queue support, with built-in support for delays between actions.
	 */
	class RENDERGRID_API FRenderGridGenericExecutionQueue : public TSharedFromThis<FRenderGridGenericExecutionQueue>, public FTickableGameObject
	{
	public:
		//~ Begin FTickableGameObject interface
		virtual void Tick(float DeltaTime) override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		virtual bool IsTickableWhenPaused() const override { return true; }
		virtual bool IsTickableInEditor() const override { return true; }
		virtual TStatId GetStatId() const override
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderGridGenericExecutionQueue, STATGROUP_Tickables);
		}
		//~ End FTickableGameObject interface

	public:
		/** Queues the given action. */
		void Add(const FRenderGridGenericExecutionQueueEntry& Entry) { QueuedEntries.AddTail(Entry); }

		/** Queues the given delay. */
		void Delay(const FRenderGridGenericExecutionQueueDelay& Delay) { Add(FRenderGridGenericExecutionQueueActionReturningDelay::CreateLambda([Delay]() -> FRenderGridGenericExecutionQueueDelay { return Delay; })); }

		/** Queues the given delay, which will wait for the given number of frames. */
		void DelayFrames(const int64 Frames) { Delay(FRenderGridGenericExecutionQueueDelay::Frames(Frames)); }

		/** Queues the given delay, which will wait for the given number of seconds. */
		void DelaySeconds(const double Seconds) { Delay(FRenderGridGenericExecutionQueueDelay::Seconds(Seconds)); }

		/** Queues the given delay, which will wait for the given number of frames or seconds, whatever takes the longest. */
		void DelayFramesOrSeconds(const int64 Frames, const double Seconds) { Delay(FRenderGridGenericExecutionQueueDelay::FramesOrSeconds(Frames, Seconds)); }

		/** Starts the execution of this queue. */
		void Start();

		/** Stops (pauses) the execution of this queue, this can be resumed by calling the Start function again. Currently queued up delays will continue to run/expire. */
		void Stop();

		/** Returns true if Start() has been called and Stop() hasn't been called yet. */
		bool IsRunning() { return bRunning; }

	protected:
		/** Executes the next delay (if there are any), otherwise it executes the next entry (action). */
		void ExecuteNext();

		/** Executes the next delay, returns true if it found any, returns false if there were no queued up delays. */
		bool ExecuteNextDelay();

		/** Executes the next entry (action), returns true if it found and executed an entry, returns false if there were no queued up entries. */
		bool ExecuteNextEntry();

		/** Adds the delay to the queued delays. */
		void QueueDelay(const FRenderGridGenericExecutionQueueDelay& Delay) { QueuedDelays.AddTail(Delay); }

	protected:
		/** The queued up entries (actions). */
		TDoubleLinkedList<FRenderGridGenericExecutionQueueEntry> QueuedEntries;

		/** The queued up delays. */
		TDoubleLinkedList<FRenderGridGenericExecutionQueueDelay> QueuedDelays;

	protected:
		/** Whether it has started (and hasn't been stopped/paused yet). This means that if this is true, Start() has been called, and Stop() hasn't been called yet since then. */
		bool bRunning = false;

		/** Whether it's currently executing a delay or an entry (action). False means there were no delays and entries queued up anymore. */
		bool bExecuting = false;

		/** The number of frames the current delay has to wait for. */
		int64 DelayRemainingFrames = 0;

		/** The number of seconds the current delay has to wait for. */
		double DelayRemainingSeconds = 0.0f;

		/** The TFuture it's waiting for (if any). */
		TSharedFuture<void> DelayRemainingFuture;

		/** The TFuture it's waiting for (if any), that can return a delay. */
		TSharedFuture<FRenderGridGenericExecutionQueueDelay> DelayRemainingFutureReturningDelay;
	};
}
