// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "RenderGridRemoteControlUtils.generated.h"


UCLASS()
class RENDERGRID_API URenderGridRemoteControlUtils : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsByte(const FString& Json, const uint8 DefaultValue, bool& bSuccess, uint8& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsInt32(const FString& Json, const int32 DefaultValue, bool& bSuccess, int32& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsInt64(const FString& Json, const int64 DefaultValue, bool& bSuccess, int64& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control double"))
	static void ParseJsonAsFloat(const FString& Json, const double DefaultValue, bool& bSuccess, double& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsBoolean(const FString& Json, bool DefaultValue, bool& bSuccess, bool& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsString(const FString& Json, const FString& DefaultValue, bool& bSuccess, FString& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsName(const FString& Json, const FName& DefaultValue, bool& bSuccess, FName& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsText(const FString& Json, const FText& DefaultValue, bool& bSuccess, FText& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsObjectReference(const FString& Json, UObject* DefaultValue, bool& bSuccess, UObject*& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsClassReference(const FString& Json, UClass* DefaultValue, bool& bSuccess, UClass*& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsStruct(const FString& Json, const FInstancedStruct& DefaultValue, bool& bSuccess, FInstancedStruct& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsVector(const FString& Json, const FVector& DefaultValue, bool& bSuccess, FVector& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsRotator(const FString& Json, const FRotator& DefaultValue, bool& bSuccess, FRotator& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsTransform(const FString& Json, const FTransform& DefaultValue, bool& bSuccess, FTransform& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsColor(const FString& Json, const FColor& DefaultValue, bool& bSuccess, FColor& Value);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ParseJsonAsLinearColor(const FString& Json, const FLinearColor& DefaultValue, bool& bSuccess, FLinearColor& Value);


	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ByteToJson(uint8 Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void Int32ToJson(int32 Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void Int64ToJson(int64 Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control double"))
	static void FloatToJson(double Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void BooleanToJson(bool Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void StringToJson(const FString& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void NameToJson(const FName& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void TextToJson(const FText& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ObjectReferenceToJson(UObject* Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ClassReferenceToJson(UClass* Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void StructToJson(const FInstancedStruct& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void VectorToJson(const FVector& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void RotatorToJson(const FRotator& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void TransformToJson(const FTransform& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void ColorToJson(const FColor& Value, FString& Json);

	UFUNCTION(BlueprintPure, Category="Render Grid Utils", Meta=(Keywords="remote control"))
	static void LinearColorToJson(const FLinearColor& Value, FString& Json);
};
