// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IImageWrapper.h"
#include "RenderGridUtils.generated.h"


class URenderGrid;
class UTexture2D;
class FJsonValue;


/**
 * This struct keeps track of the values of the GEngine framerate settings before new values were applied, so we can rollback to the previous state.
 */
USTRUCT()
struct RENDERGRID_API FRenderGridPreviousEngineFpsSettings
{
	GENERATED_BODY()

public:
	/** Whether the values have been set or not. */
	UPROPERTY()
	bool bHasBeenSet = false;

	/** The previous value of GEngine->bUseFixedFrameRate. */
	UPROPERTY()
	bool bUseFixedFrameRate = false;

	/** The previous value of GEngine->bForceDisableFrameRateSmoothing. */
	UPROPERTY()
	bool bForceDisableFrameRateSmoothing = false;

	/** The previous value of GEngine->GetMaxFPS(). */
	UPROPERTY()
	float MaxFps = 0;

	/** The previous value of console variable "r.VSync". */
	UPROPERTY()
	bool bVSync = false;

	/** The previous value of console variable "r.VSyncEditor". */
	UPROPERTY()
	bool bVSyncEditor = false;

	/** The previous value of UEditorPerformanceSettings->bThrottleCPUWhenNotForeground. */
	UPROPERTY()
	bool bThrottleCPUWhenNotForeground = false;
};


namespace UE::RenderGrid::Private
{
	/**
	 * A class containing static utility functions for the RenderGrid module.
	 */
	class RENDERGRID_API FRenderGridUtils
	{
	public:
		/**
		 * Returns true if the given file is likely a valid image.
		 */
		static bool IsImage(const FString& ImagePath);

		/**
		 * Loads an image from the disk, tries to automatically figure out the correct image format.
		 *
		 * Returns NULL if it fails.
		 *
		 * Will re-use the given Texture2D if possible, bOutReusedGivenTexture2D will be true if it was possible.
		 */
		static UTexture2D* GetImage(const FString& ImagePath, UTexture2D* Texture2D, bool& bOutReusedGivenTexture2D);


		/**
		 * Converts bytes into a texture.
		 *
		 * Returns NULL if it fails.
		 */
		static UTexture2D* BytesToImage(const TArray<uint8>& ByteArray, const EImageFormat ImageFormat);

		/**
		 * Converts bytes into an texture.
		 *
		 * Returns NULL if it fails.
		 *
		 * Will re-use the given Texture2D if possible, bOutReusedGivenTexture2D will be true if it was possible.
		 */
		static UTexture2D* BytesToExistingImage(const TArray<uint8>& ByteArray, const EImageFormat ImageFormat, UTexture2D* Texture2D, bool& bOutReusedGivenTexture2D);

		/**
		 * Converts texture data into a texture.
		 *
		 * Returns NULL if it fails.
		 */
		static UTexture2D* DataToTexture2D(int32 Width, int32 Height, const void* Src, SIZE_T Count);

		/**
		 * Converts texture data into an texture.
		 *
		 * Returns NULL if it fails.
		 *
		 * Will re-use the given Texture2D if possible, bOutReusedGivenTexture2D will be true if it was possible.
		 */
		static UTexture2D* DataToExistingTexture2D(int32 Width, int32 Height, const void* Src, SIZE_T Count, UTexture2D* Texture2D, bool& bOutReusedGivenTexture2D);


		/**
		 * Returns the paths of the files that exist in the given directory path.
		 */
		static TArray<FString> GetFiles(const FString& Directory, const bool bRecursive = false);

		/**
		 * Returns the data of the file, returns an empty byte array if the file doesn't exist.
		 *
		 * Note: Can only open files of 2GB and smaller, will return an empty byte array if it is bigger than 2GB.
		 */
		static TArray<uint8> GetFileData(const FString& File);


		/**
		 * Deletes all files and directories in the given directory, including the given directory.
		 */
		static void DeleteDirectory(const FString& Directory);

		/**
		 * Deletes all files and directories in the given directory, won't delete the given directory.
		 */
		static void EmptyDirectory(const FString& Directory);


		/**
		 * Returns a normalized directory path.
		 */
		static FString NormalizeOutputDirectory(const FString& OutputDirectory);


		/**
		 * Returns a job ID with all invalid characters removed from it. Returns an empty string if it contains no valid characters.
		 */
		static FString PurgeJobIdOrReturnEmptyString(const FString& JobId);

		/**
		 * Returns a job ID with all invalid characters removed from it. Returns string "0" if it contains no valid characters.
		 */
		static FString PurgeJobId(const FString& JobId);

		/**
		 * Returns a job ID with all invalid characters removed from it. Returns a newly generated unique ID if it contains no valid characters.
		 * The generated string is only unique relative to the job IDs that currently exist in the given RenderGrid.
		 */
		static FString PurgeJobIdOrGenerateUniqueId(URenderGrid* Grid, const FString& JobId);

		/**
		 * Returns a job name with all invalid characters removed from it. Returns an empty string if it contains no valid characters.
		 */
		static FString PurgeJobName(const FString& JobName);

		/**
		 * Returns a job output directory that's converted to a full path (if it's currently relative) and that also has the project directory path replaced by "{project_dir}" (if the given path contains that string).
		 */
		static FString NormalizeJobOutputDirectory(const FString& NewOutputDirectory);

		/**
		 * Returns a job output directory that's a usable valid filesystem path. Replaces string "{project_dir}" by the project directory.
		 */
		static FString DenormalizeJobOutputDirectory(const FString& NormalizedOutputDirectory);


		/**
		 * Converts the given FJsonValue to a string.
		 */
		static FString ToJsonString(const TSharedPtr<FJsonValue>& Value, const bool PrettyPrint = false);

		/**
		 * Parses the given JSON string and returns the FJsonValue. Will return an invalid pointer if the given JSON couldn't be parsed.
		 */
		static TSharedPtr<FJsonValue> ParseJson(const FString& Json);


		/**
		 * Converts the low-level FProperty binary data to a clean JSON string.
		 */
		static FString GetRemoteControlValueJsonFromBytes(const TArray<uint8>& ValueBytes, const bool PrettyPrint = false);

		/**
		 * Converts a clean JSON string to low-level FProperty binary data.
		 */
		static TArray<uint8> GetRemoteControlValueBytesFromJson(const TArray<uint8>& OldValueBytes, const FString& NewValueJson);


		/**
		 * Disables the current FPS limiting, returns the previous settings which allow you to revert back to the previous state.
		 */
		static FRenderGridPreviousEngineFpsSettings DisableFpsLimit();

		/**
		 * Reverts back to the previous (given) state.
		 */
		static void RestoreFpsLimit(const FRenderGridPreviousEngineFpsSettings& Settings);
	};
}
