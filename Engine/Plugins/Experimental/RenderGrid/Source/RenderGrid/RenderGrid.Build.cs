// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RenderGrid : ModuleRules
{
	public RenderGrid(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		//PCHUsage = PCHUsageMode.NoPCHs;
		//bUseUnity = false;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RemoteControl",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Json",
				"LevelSequence",
				"LevelSequenceEditor",
				"MovieRenderPipelineCore",
				"MovieRenderPipelineEditor",
				"MovieRenderPipelineRenderPasses",
				"MovieScene",
				"Serialization",
				"UnrealEd",
			}
		);
	}
}