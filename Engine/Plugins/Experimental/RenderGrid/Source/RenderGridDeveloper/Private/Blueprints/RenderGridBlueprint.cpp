// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/RenderGridBlueprint.h"

#include "EdGraphSchema_K2_Actions.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridBlueprintGeneratedClass.h"


URenderGridBlueprint::URenderGridBlueprint()
{
	RenderGrid = CreateDefaultSubobject<URenderGrid>("RenderGrid");
	RenderGrid->SetFlags(RF_DefaultSubObject | RF_Transactional);
}

UClass* URenderGridBlueprint::GetBlueprintClass() const
{
	return URenderGridBlueprintGeneratedClass::StaticClass();
}

void URenderGridBlueprint::PostLoad()
{
	Super::PostLoad();

	PropagateAllPropertiesToInstances();

	if (UbergraphPages.IsEmpty() || ((UbergraphPages.Num() == 1) && UbergraphPages[0]->Nodes.IsEmpty()))
	{
		if (!UbergraphPages.IsEmpty())
		{
			for (const TObjectPtr<UEdGraph>& Graph : UbergraphPages)
			{
				Graph->MarkAsGarbage();
				Graph->Rename(nullptr, GetTransientPackage());
			}
			UbergraphPages.Empty();
		}

		UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(this, UEdGraphSchema_K2::GN_EventGraph, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
		NewGraph->bAllowDeletion = false;

		{// create every RenderGrid blueprint event >>
			int32 i = 0;
			for (const FString& Event : URenderGrid::GetBlueprintImplementableEvents())
			{
				int32 InOutNodePosY = (i * 256) - 48;
				FKismetEditorUtilities::AddDefaultEventNode(this, NewGraph, FName(Event), URenderGrid::StaticClass(), InOutNodePosY);
				i++;
			}
		}// create every RenderGrid blueprint event <<

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(this);
		FBlueprintEditorUtils::AddUbergraphPage(this, NewGraph);
		LastEditedDocuments.AddUnique(NewGraph);

		FKismetEditorUtilities::CompileBlueprint(this, EBlueprintCompileOptions::SkipGarbageCollection);
	}

	OnChanged().RemoveAll(this);
	OnChanged().AddUObject(this, &URenderGridBlueprint::OnPostVariablesChange);

	OnPostVariablesChange(this);
}

void URenderGridBlueprint::RunOnInstances(const FRenderGridBlueprintRunOnInstancesCallback& Callback)
{
	if (UClass* MyRenderGridClass = GeneratedClass; IsValid(MyRenderGridClass))
	{
		if (URenderGrid* DefaultObject = Cast<URenderGrid>(MyRenderGridClass->GetDefaultObject(true)); IsValid(DefaultObject))
		{
			Callback.ExecuteIfBound(DefaultObject);

			TArray<UObject*> ArchetypeInstances;
			DefaultObject->GetArchetypeInstances(ArchetypeInstances);
			for (UObject* ArchetypeInstance : ArchetypeInstances)
			{
				if (URenderGrid* InstanceGrid = Cast<URenderGrid>(ArchetypeInstance); IsValid(InstanceGrid))
				{
					Callback.ExecuteIfBound(InstanceGrid);
				}
			}
		}
	}
}

void URenderGridBlueprint::Load()
{
	for (URenderGridJob* Job : RenderGrid->GetRenderGridJobsRef())
	{
		Job->Rename(nullptr, RenderGrid);
	}
}

void URenderGridBlueprint::Save()
{
	if (UClass* MyRenderGridClass = GeneratedClass; IsValid(MyRenderGridClass))
	{
		if (URenderGrid* DefaultObject = Cast<URenderGrid>(MyRenderGridClass->GetDefaultObject(true)); IsValid(DefaultObject))
		{
			for (URenderGridJob* Job : DefaultObject->GetRenderGridJobsRef())
			{
				Job->Rename(nullptr, DefaultObject);
			}
		}
	}
}

void URenderGridBlueprint::PropagateJobsToInstances()
{
	RunOnInstances(FRenderGridBlueprintRunOnInstancesCallback::CreateLambda([this](URenderGrid* Instance)
	{
		Instance->CopyJobs(RenderGrid);
	}));
}

void URenderGridBlueprint::PropagateAllPropertiesExceptJobsToInstances()
{
	URenderGrid* CDO = GetRenderGridClassDefaultObject();
	RunOnInstances(FRenderGridBlueprintRunOnInstancesCallback::CreateLambda([this, CDO](URenderGrid* Instance)
	{
		Instance->CopyAllPropertiesExceptJobs(RenderGrid);
		if (IsValid(CDO))
		{
			Instance->CopyAllUserVariables(CDO);
		}
	}));
}

void URenderGridBlueprint::PropagateAllPropertiesToInstances()
{
	URenderGrid* CDO = GetRenderGridClassDefaultObject();
	RunOnInstances(FRenderGridBlueprintRunOnInstancesCallback::CreateLambda([this, CDO](URenderGrid* Instance)
	{
		Instance->CopyAllProperties(RenderGrid);
		if (IsValid(CDO))
		{
			Instance->CopyAllUserVariables(CDO);
		}
	}));
}

void URenderGridBlueprint::PropagateJobsToAsset(URenderGrid* Instance)
{
	check(IsValid(Instance));

	RenderGrid->CopyJobs(Instance);
}

void URenderGridBlueprint::PropagateAllPropertiesExceptJobsToAsset(URenderGrid* Instance)
{
	check(IsValid(Instance));

	RenderGrid->CopyAllPropertiesExceptJobs(Instance);
	if (URenderGrid* CDO = GetRenderGridClassDefaultObject(); IsValid(CDO))
	{
		CDO->CopyAllUserVariables(Instance);
	}
}

void URenderGridBlueprint::PropagateAllPropertiesToAsset(URenderGrid* Instance)
{
	check(IsValid(Instance));

	RenderGrid->CopyAllProperties(Instance);
	if (URenderGrid* CDO = GetRenderGridClassDefaultObject(); IsValid(CDO))
	{
		CDO->CopyAllUserVariables(Instance);
	}
}

URenderGrid* URenderGridBlueprint::GetRenderGridWithBlueprintGraph() const
{
	URenderGrid* Result = GetRenderGrid();
	const_cast<URenderGridBlueprint*>(this)->RunOnInstances(FRenderGridBlueprintRunOnInstancesCallback::CreateLambda([&Result](URenderGrid* Instance)
	{
		if (!IsValid(Result) || Result->HasAnyFlags(RF_ClassDefaultObject | RF_DefaultSubObject))
		{
			Result = Instance;
		}
	}));
	return Result;
}

URenderGrid* URenderGridBlueprint::GetRenderGridClassDefaultObject() const
{
	if (UClass* MyRenderGridClass = GeneratedClass; IsValid(MyRenderGridClass))
	{
		if (URenderGrid* DefaultObject = Cast<URenderGrid>(MyRenderGridClass->GetDefaultObject(true)); IsValid(DefaultObject))
		{
			return DefaultObject;
		}
	}
	return nullptr;
}

void URenderGridBlueprint::OnPostVariablesChange(UBlueprint* InBlueprint)
{
	if (InBlueprint != this)
	{
		return;
	}

	bool bFoundChange = false;
	for (FBPVariableDescription& NewVariable : NewVariables)
	{
		uint64 PreviousPropertyFlags = NewVariable.PropertyFlags;
		bFoundChange = bFoundChange || NewVariable.HasMetaData(FBlueprintMetadata::MD_ExposeOnSpawn);

		if ((NewVariable.PropertyFlags & CPF_DisableEditOnInstance) == 0)// if [Instance Editable]
		{
			NewVariable.PropertyFlags |= CPF_BlueprintReadOnly;// set [Blueprint Read Only] to true
		}
		NewVariable.PropertyFlags &= ~CPF_ExposeOnSpawn;
		NewVariable.PropertyFlags |= CPF_Transient;
		NewVariable.RemoveMetaData(FBlueprintMetadata::MD_ExposeOnSpawn);

		bFoundChange = bFoundChange || (NewVariable.PropertyFlags != PreviousPropertyFlags);
	}

	if (bFoundChange)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(this);
	}
}
