// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/RenderGridBlueprintCompiler.h"

#include "RenderGrid/RenderGrid.h"
#include "Stats/StatsHierarchical.h"

#define LOCTEXT_NAMESPACE "RenderGridBlueprintCompiler"


bool UE::RenderGrid::FRenderGridBlueprintCompiler::CanCompile(const UBlueprint* Blueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	return (Blueprint && Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(URenderGrid::StaticClass()));
}

void UE::RenderGrid::FRenderGridBlueprintCompiler::Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRenderGridBlueprintCompilerContext Compiler(Blueprint, Results, CompileOptions);
	Compiler.Compile();
}


#undef LOCTEXT_NAMESPACE
