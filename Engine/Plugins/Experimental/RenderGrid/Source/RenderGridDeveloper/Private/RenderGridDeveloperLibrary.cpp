// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGridDeveloperLibrary.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprints/RenderGridBlueprint.h"
#include "RenderGrid/RenderGrid.h"


TArray<URenderGridBlueprint*> URenderGridDeveloperLibrary::GetAllRenderGridBlueprintAssets()
{
	FARFilter Filter;
	Filter.bIncludeOnlyOnDiskAssets = false;
	Filter.ClassPaths = {URenderGridBlueprint::StaticClass()->GetClassPathName()};
	Filter.bRecursivePaths = true;

	TArray<FAssetData> AssetsData;
	IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.GetAssets(Filter, AssetsData);

	TArray<URenderGridBlueprint*> Result;
	for (const FAssetData& AssetData : AssetsData)
	{
		if (URenderGridBlueprint* RenderGridBlueprint = Cast<URenderGridBlueprint>(AssetData.GetAsset()); IsValid(RenderGridBlueprint))
		{
			Result.Add(RenderGridBlueprint);
		}
	}
	return Result;
}

TArray<URenderGrid*> URenderGridDeveloperLibrary::GetAllRenderGridAssets()
{
	TArray<URenderGrid*> Result;
	for (URenderGridBlueprint* RenderGridBlueprint : GetAllRenderGridBlueprintAssets())
	{
		if (URenderGrid* RenderGrid = RenderGridBlueprint->GetRenderGridWithBlueprintGraph(); IsValid(RenderGrid))
		{
			Result.Add(RenderGrid);
		}
	}
	return Result;
}


URenderGridBlueprint* URenderGridDeveloperLibrary::GetRenderGridBlueprintAsset(const FString& ObjectPath)
{
	if (FSoftObjectPath AssetPath = FSoftObjectPath(ObjectPath); AssetPath.IsValid())
	{
		if (URenderGridBlueprint* RenderGridBlueprint = Cast<URenderGridBlueprint>(AssetPath.TryLoad()); IsValid(RenderGridBlueprint))
		{
			return RenderGridBlueprint;
		}
	}
	return nullptr;
}

URenderGrid* URenderGridDeveloperLibrary::GetRenderGridAsset(const FString& ObjectPath)
{
	if (URenderGridBlueprint* RenderGridBlueprint = GetRenderGridBlueprintAsset(ObjectPath); IsValid(RenderGridBlueprint))
	{
		if (URenderGrid* RenderGrid = RenderGridBlueprint->GetRenderGridWithBlueprintGraph(); IsValid(RenderGrid))
		{
			return RenderGrid;
		}
	}
	return nullptr;
}
