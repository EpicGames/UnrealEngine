// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGridDeveloperModule.h"

#define LOCTEXT_NAMESPACE "RenderGridDeveloperModule"


void UE::RenderGrid::Private::FRenderGridDeveloperModule::StartupModule() {}
void UE::RenderGrid::Private::FRenderGridDeveloperModule::ShutdownModule() {}


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::RenderGrid::Private::FRenderGridDeveloperModule, RenderGridDeveloper)
