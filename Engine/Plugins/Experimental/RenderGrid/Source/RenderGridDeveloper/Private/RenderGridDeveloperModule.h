// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRenderGridDeveloperModule.h"


namespace UE::RenderGrid::Private
{
	/**
	 * The implementation of the IRenderGridDeveloperModule interface.
	 */
	class FRenderGridDeveloperModule : public IRenderGridDeveloperModule
	{
	public:
		//~ Begin IModuleInterface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface
	};
}
