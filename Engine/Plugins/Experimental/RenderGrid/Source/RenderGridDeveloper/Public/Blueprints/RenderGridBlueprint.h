// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUtilityBlueprint.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGridBlueprint.generated.h"


/**
 * A UBlueprint child class for the RenderGrid modules.
 *
 * Required in order for a RenderGrid to be able to have a blueprint graph.
 */
UCLASS(BlueprintType, Meta=(IgnoreClassThumbnail, DontUseGenericSpawnObject="true"))
class RENDERGRIDDEVELOPER_API URenderGridBlueprint : public UEditorUtilityBlueprint
{
	GENERATED_BODY()

public:
	URenderGridBlueprint();

	//~ Begin UBlueprint Interface
	virtual bool SupportedByDefaultBlueprintFactory() const override { return false; }
	virtual bool AlwaysCompileOnLoad() const override { return true; }
	virtual bool IsValidForBytecodeOnlyRecompile() const override { return false; }
	virtual bool SupportsGlobalVariables() const override { return true; }
	virtual bool SupportsLocalVariables() const override { return true; }
	virtual bool SupportsFunctions() const override { return true; }
	virtual bool SupportsMacros() const override { return true; }
	virtual bool SupportsDelegates() const override { return true; }
	virtual bool SupportsEventGraphs() const override { return true; }
	virtual bool SupportsAnimLayers() const override { return false; }
	virtual bool ShouldBeMarkedDirtyUponTransaction() const override { return false; }
	virtual UClass* GetBlueprintClass() const override;
	virtual void PostLoad() override;
	//~ End UBlueprint Interface

private:
	DECLARE_DELEGATE_OneParam(FRenderGridBlueprintRunOnInstancesCallback, URenderGrid* /*Instance*/);
	void RunOnInstances(const FRenderGridBlueprintRunOnInstancesCallback& Callback);

public:
	void Load();
	void Save();

	void PropagateJobsToInstances();
	void PropagateAllPropertiesExceptJobsToInstances();
	void PropagateAllPropertiesToInstances();

	void PropagateJobsToAsset(URenderGrid* Instance);
	void PropagateAllPropertiesExceptJobsToAsset(URenderGrid* Instance);
	void PropagateAllPropertiesToAsset(URenderGrid* Instance);

public:
	/** Returns the RenderGrid reference that this RenderGrid asset contains. This is simply the data representation of the render grid, meaning that it won't contain a blueprint graph or any user code. */
	UFUNCTION(BlueprintPure, Category="Render Grid")
	URenderGrid* GetRenderGrid() const { return RenderGrid; }

	/** Returns the RenderGrid reference that this RenderGrid asset contains. This will be the subclass of the blueprint class, meaning it will contain a blueprint graph. */
	UFUNCTION(BlueprintPure, Category="Render Grid")
	URenderGrid* GetRenderGridWithBlueprintGraph() const;

	/** Returns the RenderGrid reference that this RenderGrid asset contains. This will be the default object of the subclass of the blueprint class, meaning it will contain a blueprint graph. */
	UFUNCTION(BlueprintPure, Category="Render Grid")
	URenderGrid* GetRenderGridClassDefaultObject() const;

private:
	virtual void OnPostVariablesChange(UBlueprint* InBlueprint);

private:
	UPROPERTY()
	TObjectPtr<URenderGrid> RenderGrid;
};
