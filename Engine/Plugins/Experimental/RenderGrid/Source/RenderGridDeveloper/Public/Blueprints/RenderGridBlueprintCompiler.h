// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetCompilerModule.h"
#include "KismetCompiler.h"


namespace UE::RenderGrid
{
	/**
	 * A IBlueprintCompiler child class for the RenderGrid modules.
	 *
	 * Required in order for a RenderGrid to be able to have a blueprint graph.
	 */
	class RENDERGRIDDEVELOPER_API FRenderGridBlueprintCompiler : public IBlueprintCompiler
	{
	public:
		//~ Begin IBlueprintCompiler Interface
		virtual bool CanCompile(const UBlueprint* Blueprint) override;
		virtual void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) override;
		//~ End IBlueprintCompiler Interface
	};


	class RENDERGRIDDEVELOPER_API FRenderGridBlueprintCompilerContext : public FKismetCompilerContext
	{
	public:
		FRenderGridBlueprintCompilerContext(UBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
			: FKismetCompilerContext(SourceSketch, InMessageLog, InCompilerOptions)
		{}
	};
}
