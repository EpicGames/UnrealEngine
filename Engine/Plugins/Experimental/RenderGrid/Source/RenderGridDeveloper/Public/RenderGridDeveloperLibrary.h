// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "RenderGridDeveloperLibrary.generated.h"


class URenderGrid;
class URenderGridBlueprint;


UCLASS(meta=(ScriptName="RenderGridDeveloperLibrary"))
class RENDERGRIDDEVELOPER_API URenderGridDeveloperLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Returns all render grid assets that currently exist in the project (on disk and in memory).
	 * Will load the render grid assets in that are currently unloaded.
	 * 
	 * This is a slow operation, so avoid doing this every tick.
	 */
	static TArray<URenderGridBlueprint*> GetAllRenderGridBlueprintAssets();

	/**
	 * Returns all render grid assets that currently exist in the project (on disk and in memory).
	 * Will load the render grid assets in that are currently unloaded.
	 * 
	 * This is a slow operation, so avoid doing this every tick.
	 */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	static TArray<URenderGrid*> GetAllRenderGridAssets();


	/**
	 * Returns the given render grid asset that exists at the given object path (whether it's on disk or in memory).
	 * Will load the render grid asset if it's currently unloaded.
	 * 
	 * This is a potentially slow operation, so avoid doing this every tick.
	 */
	static URenderGridBlueprint* GetRenderGridBlueprintAsset(const FString& ObjectPath);

	/**
	 * Returns the given render grid asset that exists at the given object path (whether it's on disk or in memory).
	 * Will load the render grid asset if it's currently unloaded.
	 * 
	 * This is a potentially slow operation, so avoid doing this every tick.
	 */
	UFUNCTION(BlueprintCallable, Category="Render Grid")
	static URenderGrid* GetRenderGridAsset(const FString& ObjectPath);
};
