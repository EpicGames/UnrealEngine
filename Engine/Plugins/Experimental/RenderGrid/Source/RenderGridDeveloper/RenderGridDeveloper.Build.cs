// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RenderGridDeveloper : ModuleRules
{
	public RenderGridDeveloper(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		//PCHUsage = PCHUsageMode.NoPCHs;
		//bUseUnity = false;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Blutility",
				"Core",
				"CoreUObject",
				"Engine",
				"KismetCompiler", 
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"Engine",
				"GraphEditor",
				"RenderGrid",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);
	}
}