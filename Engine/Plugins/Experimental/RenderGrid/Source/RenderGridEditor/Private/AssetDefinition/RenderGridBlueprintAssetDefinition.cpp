// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGridBlueprintAssetDefinition.h"
#include "Blueprints/RenderGridBlueprint.h"
#include "RenderGridEditorModule.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "RenderGridBlueprintAssetDefinition"


FText URenderGridBlueprintAssetDefinition::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDefinition_DisplayName_RenderGrid", "Render Grid");
}

FLinearColor URenderGridBlueprintAssetDefinition::GetAssetColor() const
{
	return FLinearColor(FColor(200, 80, 80));
}

TSoftClassPtr<UObject> URenderGridBlueprintAssetDefinition::GetAssetClass() const
{
	return URenderGridBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> URenderGridBlueprintAssetDefinition::GetAssetCategories() const
{
	static const auto Categories = {EAssetCategoryPaths::Misc};
	return Categories;
}

FAssetSupportResponse URenderGridBlueprintAssetDefinition::CanLocalize(const FAssetData& InAsset) const
{
	return FAssetSupportResponse::NotSupported();
}

EAssetCommandResult URenderGridBlueprintAssetDefinition::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<FAssetData> OutAssetsThatFailedToLoad;
	for (URenderGridBlueprint* RenderGridBlueprint : OpenArgs.LoadObjects<URenderGridBlueprint>({}, &OutAssetsThatFailedToLoad))
	{
		bool bLetOpen = true;
		if (!RenderGridBlueprint->SkeletonGeneratedClass || !RenderGridBlueprint->GeneratedClass)
		{
			bLetOpen = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("FailedToLoadBlueprintWithContinue", "Blueprint could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed! Do you want to continue (it can crash the editor)?"));
		}
		if (bLetOpen)
		{
			UE::RenderGrid::IRenderGridEditorModule::Get().CreateRenderGridEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, RenderGridBlueprint);
		}
	}

	for (const FAssetData& UnableToLoadAsset : OutAssetsThatFailedToLoad)
	{
		FMessageDialog::Open(EAppMsgType::Ok,
			FText::Format(
				LOCTEXT("FailedToLoadBlueprint", "Blueprint '{0}' could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed!"),
				FText::FromName(UnableToLoadAsset.PackagePath)
			)
		);
	}

	return EAssetCommandResult::Handled;
}


#undef LOCTEXT_NAMESPACE
