// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "RenderGridBlueprintAssetDefinition.generated.h"


/**
 * This class adds support for the RenderGrid (RenderGridBlueprint) class to the AssetDefinition module.
 */
UCLASS()
class URenderGridBlueprintAssetDefinition : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition Interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	//~ End UAssetDefinition interface
};
