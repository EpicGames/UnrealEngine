// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/RenderGridApplicationModeBase.h"
#include "BlueprintModes/RenderGridApplicationModes.h"
#include "IRenderGridEditor.h"


UE::RenderGrid::Private::FRenderGridApplicationModeBase::FRenderGridApplicationModeBase(TSharedPtr<IRenderGridEditor> InRenderGridEditor, FName InModeName)
	: FBlueprintEditorApplicationMode(InRenderGridEditor, InModeName, FRenderGridApplicationModes::GetLocalizedMode, false, false)
	, BlueprintEditorWeakPtr(InRenderGridEditor)
{}

URenderGridBlueprint* UE::RenderGrid::Private::FRenderGridApplicationModeBase::GetBlueprint() const
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		return BlueprintEditor->GetRenderGridBlueprint();
	}
	return nullptr;
}

TSharedPtr<UE::RenderGrid::IRenderGridEditor> UE::RenderGrid::Private::FRenderGridApplicationModeBase::GetBlueprintEditor() const
{
	return BlueprintEditorWeakPtr.Pin();
}
