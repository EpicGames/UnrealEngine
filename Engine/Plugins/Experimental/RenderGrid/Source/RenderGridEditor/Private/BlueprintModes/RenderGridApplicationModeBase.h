// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintEditorModes.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"


class URenderGridBlueprint;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * This is the base class for the render grid editor application modes.
	 * 
	 * It contains functionality that's shared between all the render grid editor application modes.
	 */
	class FRenderGridApplicationModeBase : public FBlueprintEditorApplicationMode
	{
	public:
		FRenderGridApplicationModeBase(TSharedPtr<IRenderGridEditor> InRenderGridEditor, FName InModeName);

	protected:
		/** Returns the RenderGridBlueprint of the editor that was given to the constructor. */
		URenderGridBlueprint* GetBlueprint() const;

		/** Returns the editor that was given to the constructor. */
		TSharedPtr<IRenderGridEditor> GetBlueprintEditor() const;

	protected:
		/** The editor that was given to the constructor. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** Set of spawnable tabs in the mode. */
		FWorkflowAllowedTabSet TabFactories;
	};
}
