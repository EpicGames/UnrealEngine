// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/RenderGridApplicationModeListing.h"
#include "BlueprintModes/RenderGridApplicationModes.h"

#include "BlueprintEditorSharedTabFactories.h"
#include "BlueprintEditorTabs.h"

#include "IRenderGridEditor.h"
#include "IRenderGridEditorModule.h"

#include "SBlueprintEditorToolbar.h"

#include "TabFactory/RenderGridJobListTabSummoner.h"
#include "TabFactory/RenderGridPropertiesTabSummoner.h"
#include "TabFactory/RenderGridJobPropertiesTabSummoner.h"
#include "TabFactory/RenderGridViewerTabSummoner.h"

#include "Toolkit/RenderGridEditorToolbar.h"

#define LOCTEXT_NAMESPACE "RenderGridListingMode"


UE::RenderGrid::Private::FRenderGridApplicationModeListing::FRenderGridApplicationModeListing(TSharedPtr<IRenderGridEditor> InRenderGridEditor)
	: FRenderGridApplicationModeBase(InRenderGridEditor, FRenderGridApplicationModes::ListingMode)
{
	// Override the default created category here since "Listing Editor" sounds awkward
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_RenderGridListing", "Render Grid Listing"));

	TabLayout = FTabManager::NewLayout("RenderGridBlueprintEditor_Listing_Layout_v1_000")
		->AddArea(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.5f)
				->SetOrientation(Orient_Horizontal)
				->Split(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(1.f)
					->SetForegroundTab(FRenderGridJobListTabSummoner::TabID)
					->AddTab(FRenderGridJobListTabSummoner::TabID, ETabState::OpenedTab)
				)
			)
			->Split(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.5f)
				->SetOrientation(Orient_Horizontal)
				->Split(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.25f)
					->SetForegroundTab(FRenderGridPropertiesTabSummoner::TabID)
					->AddTab(FRenderGridPropertiesTabSummoner::TabID, ETabState::OpenedTab)
				)
				->Split(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.5f)
					->SetForegroundTab(FRenderGridViewerTabSummoner::TabID)
					->AddTab(FRenderGridViewerTabSummoner::TabID, ETabState::OpenedTab)
				)
				->Split(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->SetSizeCoefficient(0.25f)
					->SetForegroundTab(FRenderGridJobPropertiesTabSummoner::TabID)
					->AddTab(FRenderGridJobPropertiesTabSummoner::TabID, ETabState::OpenedTab)
				)
			)
		);

	// Register Tab Factories
	TabFactories.RegisterFactory(MakeShareable(new FRenderGridJobListTabSummoner(InRenderGridEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FRenderGridPropertiesTabSummoner(InRenderGridEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FRenderGridViewerTabSummoner(InRenderGridEditor)));
	TabFactories.RegisterFactory(MakeShareable(new FRenderGridJobPropertiesTabSummoner(InRenderGridEditor)));

	//Make sure we start with our existing list of extenders instead of creating a new one
	IRenderGridEditorModule& RenderGridEditorModule = IRenderGridEditorModule::Get();
	ToolbarExtender = RenderGridEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders();

	InRenderGridEditor->GetRenderGridToolbarBuilder()->AddRenderGridBlueprintEditorModesToolbar(ToolbarExtender);

	if (UToolMenu* Toolbar = InRenderGridEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InRenderGridEditor->GetRenderGridToolbarBuilder()->AddListingModeToolbar(Toolbar);

		InRenderGridEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
	}
}

void UE::RenderGrid::Private::FRenderGridApplicationModeListing::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<IRenderGridEditor> BP = GetBlueprintEditor();

	BP->RegisterToolbarTab(InTabManager.ToSharedRef());
	BP->PushTabFactories(TabFactories);
}

void UE::RenderGrid::Private::FRenderGridApplicationModeListing::PreDeactivateMode()
{
	// prevents: FRenderGridApplicationModeBase::PreDeactivateMode();
}

void UE::RenderGrid::Private::FRenderGridApplicationModeListing::PostActivateMode()
{
	// prevents: FRenderGridApplicationModeBase::PostActivateMode();
}


#undef LOCTEXT_NAMESPACE
