// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintModes/RenderGridApplicationModeBase.h"


namespace UE::RenderGrid::Private
{
	/**
	 * This is the application mode for the render grid editor listing functionality (the list of render grid jobs, with the render previews, the render grid job properties, etc).
	 */
	class FRenderGridApplicationModeListing : public FRenderGridApplicationModeBase
	{
	public:
		FRenderGridApplicationModeListing(TSharedPtr<IRenderGridEditor> InRenderGridEditor);

		//~ Begin FApplicationMode interface
		virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
		virtual void PreDeactivateMode() override;
		virtual void PostActivateMode() override;
		//~ End FApplicationMode interface
	};
}
