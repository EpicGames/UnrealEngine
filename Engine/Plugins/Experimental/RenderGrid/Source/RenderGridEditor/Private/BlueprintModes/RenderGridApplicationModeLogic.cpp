// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/RenderGridApplicationModeLogic.h"
#include "BlueprintModes/RenderGridApplicationModes.h"
#include "IRenderGridEditor.h"

#include "BlueprintEditorTabs.h"
#include "SBlueprintEditorToolbar.h"
#include "Toolkit/RenderGridEditorToolbar.h"

#define LOCTEXT_NAMESPACE "RenderGridLogicMode"


UE::RenderGrid::Private::FRenderGridApplicationModeLogic::FRenderGridApplicationModeLogic(TSharedPtr<IRenderGridEditor> InRenderGridEditor)
	: FRenderGridApplicationModeBase(InRenderGridEditor, FRenderGridApplicationModes::LogicMode)
{
	// Override the default created category here since "Logic Editor" sounds awkward
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_RenderGridLogic", "Render Grid Logic"));

	TabLayout = FTabManager::NewLayout("RenderGridBlueprintEditor_Logic_Layout_v1")
		->AddArea(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.15f)
					->Split(
						FTabManager::NewStack()->SetSizeCoefficient(0.5f)
						->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
					)
					->Split(
						FTabManager::NewStack()->SetSizeCoefficient(0.5f)
						->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
					)
				)
				->Split(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.70f)
					->Split(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.80f)
						->AddTab("Document", ETabState::ClosedTab)
					)
					->Split(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.20f)
						->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab)
						->AddTab(FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab)
					)
				)
				->Split(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.15f)
					->Split(
						FTabManager::NewStack()
						->AddTab(FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab)
					)
				)
			)
		);

	//Make sure we start with new one
	ToolbarExtender = MakeShareable(new FExtender);

	InRenderGridEditor->GetRenderGridToolbarBuilder()->AddRenderGridBlueprintEditorModesToolbar(ToolbarExtender);

	if (UToolMenu* Toolbar = InRenderGridEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InRenderGridEditor->GetRenderGridToolbarBuilder()->AddLogicModeToolbar(Toolbar);

		InRenderGridEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InRenderGridEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		// disabled: InRenderGridEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
	}
}

void UE::RenderGrid::Private::FRenderGridApplicationModeLogic::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = GetBlueprintEditor();

	BP->RegisterToolbarTab(InTabManager.ToSharedRef());
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
	BP->PushTabFactories(TabFactories);
}

void UE::RenderGrid::Private::FRenderGridApplicationModeLogic::PreDeactivateMode()
{
	// prevents: FRenderGridApplicationModeBase::PreDeactivateMode();
}

void UE::RenderGrid::Private::FRenderGridApplicationModeLogic::PostActivateMode()
{
	FRenderGridApplicationModeBase::PostActivateMode();
}


#undef LOCTEXT_NAMESPACE
