// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintModes/RenderGridApplicationModeBase.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"


namespace UE::RenderGrid::Private
{
	/**
	 * This is the application mode for the render grid editor 'logic' functionality (the blueprint graph).
	 */
	class FRenderGridApplicationModeLogic : public FRenderGridApplicationModeBase
	{
	public:
		FRenderGridApplicationModeLogic(TSharedPtr<IRenderGridEditor> InRenderGridEditor);

		//~ Begin FApplicationMode interface
		virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
		virtual void PreDeactivateMode() override;
		virtual void PostActivateMode() override;
		//~ End FApplicationMode interface
	};
}
