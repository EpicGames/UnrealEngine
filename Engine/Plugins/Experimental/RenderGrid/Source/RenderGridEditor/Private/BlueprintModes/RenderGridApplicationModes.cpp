// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintModes/RenderGridApplicationModes.h"


// Mode constants
const FName UE::RenderGrid::Private::FRenderGridApplicationModes::ListingMode("ListingName");
const FName UE::RenderGrid::Private::FRenderGridApplicationModes::LogicMode("LogicName");


FText UE::RenderGrid::Private::FRenderGridApplicationModes::GetLocalizedMode(const FName InMode)
{
	static TMap<FName, FText> LocModes;
	if (LocModes.Num() == 0)
	{
		LocModes.Add(ListingMode, NSLOCTEXT("RenderGridBlueprintModes", "ListingMode", "Listing"));
		LocModes.Add(LogicMode, NSLOCTEXT("RenderGridBlueprintModes", "LogicMode", "Logic"));
	}

	check(InMode != NAME_None);
	const FText* OutDesc = LocModes.Find(InMode);
	check(OutDesc);

	return *OutDesc;
}
