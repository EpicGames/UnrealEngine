// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


namespace UE::RenderGrid::Private
{
	/**
	 * This struct contains the IDs as well as the localized text of each of the render grid editor application modes.
	 */
	struct FRenderGridApplicationModes
	{
	public:
		/** Constant for the listing mode. */
		static const FName ListingMode;

		/** Constant for the logic mode. */
		static const FName LogicMode;

	public:
		/** Returns the localized text for the given render grid editor application mode. */
		static FText GetLocalizedMode(const FName InMode);

	private:
		FRenderGridApplicationModes() = default;
	};
}
