// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/RenderGridEditorCommands.h"

#define LOCTEXT_NAMESPACE "FRenderGridEditor"


void UE::RenderGrid::Private::FRenderGridEditorCommands::RegisterCommands()
{
	UI_COMMAND(AddJob, "Add", "Creates and adds a new job instance to the grid.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::N));
	UI_COMMAND(DuplicateJob, "Duplicate", "Duplicates the selected job instances and adds them to the grid.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D));
	UI_COMMAND(DeleteJob, "Delete", "Deletes the selected job instances from the grid.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::None, EKeys::Delete));

	UI_COMMAND(BatchRenderList, "Render", "Renders the enabled job instances of the grid.", EUserInterfaceActionType::Button, FInputChord());
}


#undef LOCTEXT_NAMESPACE
