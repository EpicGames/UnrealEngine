// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"


namespace UE::RenderGrid::Private
{
	/**
	 * Contains the commands for the render grid editor.
	 */
	class FRenderGridEditorCommands : public TCommands<FRenderGridEditorCommands>
	{
	public:
		FRenderGridEditorCommands()
			: TCommands<FRenderGridEditorCommands>(TEXT("RenderGridEditor"),
				NSLOCTEXT("Contexts", "RenderGridEditor", "Render Grid Editor"),
				NAME_None,
				FAppStyle::GetAppStyleSetName())
		{}

		virtual void RegisterCommands() override;

	public:
		/** Creates and adds a new job to the grid. */
		TSharedPtr<FUICommandInfo> AddJob;

		/** Duplicates the selected job(s) and adds it to the grid. */
		TSharedPtr<FUICommandInfo> DuplicateJob;

		/** Deletes the selected job(s) from the grid. */
		TSharedPtr<FUICommandInfo> DeleteJob;

		/** Renders the enabled job(s) in batch. */
		TSharedPtr<FUICommandInfo> BatchRenderList;
	};
}
