// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RenderGridBlueprintFactory.h"
#include "Blueprints/RenderGridBlueprint.h"
#include "IContentBrowserSingleton.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridBlueprintGeneratedClass.h"

#define LOCTEXT_NAMESPACE "RenderGridBlueprintFactory"


URenderGridBlueprintFactory::URenderGridBlueprintFactory()
{
	ParentClass = URenderGrid::StaticClass();
	SupportedClass = URenderGridBlueprint::StaticClass();
	bCreateNew = true; // This factory manufacture new objects from scratch
	bEditAfterNew = true; // This factory will open the editor for each new object
}

UObject* URenderGridBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a Render Grid Blueprint, then create and init one
	check(InClass->IsChildOf(URenderGridBlueprint::StaticClass()));

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf(URenderGrid::StaticClass()))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : LOCTEXT("Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("CannotCreateRenderGridBlueprint", "Cannot create an Render Grid Blueprint based on the class '{0}'."), Args));
		return nullptr;
	}

	if (URenderGridBlueprint* RenderGridBlueprint = CastChecked<URenderGridBlueprint>(FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, InName, BPTYPE_Normal, URenderGridBlueprint::StaticClass(), URenderGridBlueprintGeneratedClass::StaticClass(), CallingContext)))
	{
		RenderGridBlueprint->PostLoad();
		return RenderGridBlueprint;
	}
	return nullptr;
}

UObject* URenderGridBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(InClass, InParent, InName, Flags, Context, Warn, NAME_None);
}

bool URenderGridBlueprintFactory::ShouldShowInNewMenu() const
{
	return true;
}

uint32 URenderGridBlueprintFactory::GetMenuCategories() const
{
	//  if wanting to show it in its own category:
	// IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	// return AssetTools.RegisterAdvancedAssetCategory("Render Grid", LOCTEXT("AssetCategoryName", "Render Grid"));

	return EAssetTypeCategories::Misc;
}


#undef LOCTEXT_NAMESPACE
