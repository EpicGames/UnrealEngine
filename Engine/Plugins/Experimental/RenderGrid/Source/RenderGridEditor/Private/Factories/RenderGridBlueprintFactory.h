// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "RenderGrid/RenderGridPropsSource.h"
#include "RenderGridBlueprintFactory.generated.h"


class URenderGridBlueprint;
class URenderGrid;


/**
 * The factory that creates URenderGridBlueprint (render grid) instances.
 */
UCLASS(MinimalAPI, HideCategories=Object)
class URenderGridBlueprintFactory : public UFactory
{
	GENERATED_BODY()

public:
	URenderGridBlueprintFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual uint32 GetMenuCategories() const override;

	/** The parent class of the created blueprint. */
	UPROPERTY(EditAnywhere, Category="Render Grid Factory", Meta = (AllowAbstract = ""))
	TSubclassOf<URenderGrid> ParentClass;
};
