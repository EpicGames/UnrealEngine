// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RenderGridPropsSourceWidgetFactoryLocal.h"
#include "UI/SRenderGridPropsLocal.h"
#include "RenderGrid/RenderGridPropsSource.h"


TSharedPtr<UE::RenderGrid::Private::SRenderGridPropsBase> UE::RenderGrid::Private::FRenderGridPropsSourceWidgetFactoryLocal::CreateInstance(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor)
{
	if (URenderGridPropsSourceLocal* InPropsSource = Cast<URenderGridPropsSourceLocal>(PropsSource))
	{
		return SNew(SRenderGridPropsLocal, BlueprintEditor, InPropsSource);
	}
	return nullptr;
}
