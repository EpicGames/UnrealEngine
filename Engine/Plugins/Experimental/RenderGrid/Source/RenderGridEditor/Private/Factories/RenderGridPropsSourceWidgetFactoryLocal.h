// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/IRenderGridPropsSourceWidgetFactory.h"


namespace UE::RenderGrid::Private
{
	/**
	 * The factory that creates props source instances for the props type "local".
	 */
	class RENDERGRIDEDITOR_API FRenderGridPropsSourceWidgetFactoryLocal final : public IRenderGridPropsSourceWidgetFactory
	{
	public:
		virtual TSharedPtr<SRenderGridPropsBase> CreateInstance(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor) override;
	};
}
