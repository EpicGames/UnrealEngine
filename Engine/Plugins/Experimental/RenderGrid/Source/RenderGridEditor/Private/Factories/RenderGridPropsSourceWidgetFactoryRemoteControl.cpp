// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RenderGridPropsSourceWidgetFactoryRemoteControl.h"
#include "UI/SRenderGridPropsRemoteControl.h"
#include "RenderGrid/RenderGridPropsSource.h"


TSharedPtr<UE::RenderGrid::Private::SRenderGridPropsBase> UE::RenderGrid::Private::FRenderGridPropsSourceWidgetFactoryRemoteControl::CreateInstance(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor)
{
	if (URenderGridPropsSourceRemoteControl* InPropsSource = Cast<URenderGridPropsSourceRemoteControl>(PropsSource))
	{
		return SNew(SRenderGridPropsRemoteControl, BlueprintEditor, InPropsSource);
	}
	return nullptr;
}
