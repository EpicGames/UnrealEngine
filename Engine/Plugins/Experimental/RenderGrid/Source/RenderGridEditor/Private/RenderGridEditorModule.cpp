// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGridEditorModule.h"

#include "Blueprints/RenderGridBlueprint.h"
#include "Commands/RenderGridEditorCommands.h"
#include "Factories/RenderGridPropsSourceWidgetFactoryLocal.h"
#include "Factories/RenderGridPropsSourceWidgetFactoryRemoteControl.h"
#include "RenderGrid/RenderGrid.h"
#include "Styles/RenderGridEditorStyle.h"
#include "Toolkit/RenderGridEditor.h"
#include "Kismet2/KismetEditorUtilities.h"

#define LOCTEXT_NAMESPACE "RenderGridEditorModule"


void UE::RenderGrid::Private::FRenderGridEditorModule::StartupModule()
{
	FRenderGridEditorStyle::Initialize();
	FRenderGridEditorStyle::ReloadTextures();
	FRenderGridEditorCommands::Register();

	MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
	ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();

	RegisterPropsSourceWidgetFactories();

	// Register to fixup newly created BPs
	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(this, URenderGrid::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(this, &FRenderGridEditorModule::HandleNewBlueprintCreated));
}

void UE::RenderGrid::Private::FRenderGridEditorModule::ShutdownModule()
{
	UnregisterPropsSourceWidgetFactories();

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	FRenderGridEditorCommands::Unregister();
	FRenderGridEditorStyle::Shutdown();
}

void UE::RenderGrid::Private::FRenderGridEditorModule::RegisterPropsSourceWidgetFactories()
{
	RegisterPropsSourceWidgetFactory(ERenderGridPropsSourceType::Local, MakeShared<FRenderGridPropsSourceWidgetFactoryLocal>());
	RegisterPropsSourceWidgetFactory(ERenderGridPropsSourceType::RemoteControl, MakeShared<FRenderGridPropsSourceWidgetFactoryRemoteControl>());
}

void UE::RenderGrid::Private::FRenderGridEditorModule::UnregisterPropsSourceWidgetFactories()
{
	UnregisterPropsSourceWidgetFactory(ERenderGridPropsSourceType::Local);
	UnregisterPropsSourceWidgetFactory(ERenderGridPropsSourceType::RemoteControl);
}

void UE::RenderGrid::Private::FRenderGridEditorModule::RegisterPropsSourceWidgetFactory(const ERenderGridPropsSourceType PropsSourceType, const TSharedPtr<IRenderGridPropsSourceWidgetFactory>& InFactory)
{
	PropsSourceWidgetFactories.Add(PropsSourceType, InFactory);
}

void UE::RenderGrid::Private::FRenderGridEditorModule::UnregisterPropsSourceWidgetFactory(const ERenderGridPropsSourceType PropsSourceType)
{
	PropsSourceWidgetFactories.Remove(PropsSourceType);
}

TSharedPtr<UE::RenderGrid::Private::SRenderGridPropsBase> UE::RenderGrid::Private::FRenderGridEditorModule::CreatePropsSourceWidget(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor)
{
	if (!PropsSource)
	{
		return nullptr;
	}

	TSharedPtr<IRenderGridPropsSourceWidgetFactory>* FactoryPtr = PropsSourceWidgetFactories.Find(PropsSource->GetType());
	if (!FactoryPtr)
	{
		return nullptr;
	}

	TSharedPtr<IRenderGridPropsSourceWidgetFactory> Factory = *FactoryPtr;
	if (!Factory)
	{
		return nullptr;
	}

	return Factory->CreateInstance(PropsSource, BlueprintEditor);
}

TSharedRef<UE::RenderGrid::IRenderGridEditor> UE::RenderGrid::Private::FRenderGridEditorModule::CreateRenderGridEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, URenderGridBlueprint* InBlueprint)
{
	TSharedRef<FRenderGridEditor> NewRenderGridEditor = MakeShared<FRenderGridEditor>();
	NewRenderGridEditor->InitRenderGridEditor(Mode, InitToolkitHost, InBlueprint);
	return NewRenderGridEditor;
}

void UE::RenderGrid::Private::FRenderGridEditorModule::HandleNewBlueprintCreated(UBlueprint* InBlueprint)
{
	if (URenderGridBlueprint* RenderGridBlueprint = Cast<URenderGridBlueprint>(InBlueprint))
	{
		RenderGridBlueprint->PostLoad();
	}
}


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::RenderGrid::Private::FRenderGridEditorModule, RenderGridEditor)
