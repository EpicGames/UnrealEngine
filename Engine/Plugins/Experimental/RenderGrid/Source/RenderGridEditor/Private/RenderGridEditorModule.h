// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRenderGridEditorModule.h"


class IToolkitHost;
class URenderGrid;
class URenderGridBlueprint;


namespace UE::RenderGrid::Private
{
	class FRenderGridEditorModule : public IRenderGridEditorModule
	{
	public:
		//~ Begin IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface interface

		//~ Begin IRenderGridEditorModule interface
		virtual TSharedRef<IRenderGridEditor> CreateRenderGridEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, URenderGridBlueprint* InBlueprint) override;

		virtual TSharedPtr<SRenderGridPropsBase> CreatePropsSourceWidget(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor) override;
		virtual const TMap<ERenderGridPropsSourceType, TSharedPtr<IRenderGridPropsSourceWidgetFactory>>& GetPropsSourceWidgetFactories() const override { return PropsSourceWidgetFactories; }
		//~ End IRenderGridEditorModule interface

		/** Gets the extensibility managers for outside entities to extend GUI of the render grid editor menus. */
		virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }

		/** Gets the extensibility managers for outside entities to extend GUI of the render grid editor toolbars. */
		virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	private:
		/** Handle a new render grid blueprint being created. */
		void HandleNewBlueprintCreated(UBlueprint* InBlueprint);

		void RegisterPropsSourceWidgetFactories();
		void UnregisterPropsSourceWidgetFactories();
		void RegisterPropsSourceWidgetFactory(const ERenderGridPropsSourceType PropsSourceType, const TSharedPtr<IRenderGridPropsSourceWidgetFactory>& InFactory);
		void UnregisterPropsSourceWidgetFactory(const ERenderGridPropsSourceType PropsSourceType);

	private:
		TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
		TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

		TMap<ERenderGridPropsSourceType, TSharedPtr<IRenderGridPropsSourceWidgetFactory>> PropsSourceWidgetFactories;
	};
}
