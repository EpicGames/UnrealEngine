// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/RenderGridEditorStyle.h"

#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"


TSharedPtr<FSlateStyleSet> UE::RenderGrid::Private::FRenderGridEditorStyle::StyleInstance = nullptr;


const ISlateStyle& UE::RenderGrid::Private::FRenderGridEditorStyle::Get()
{
	return *StyleInstance;
}

void UE::RenderGrid::Private::FRenderGridEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void UE::RenderGrid::Private::FRenderGridEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

void UE::RenderGrid::Private::FRenderGridEditorStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		if (FSlateRenderer* SlateRenderer = FSlateApplication::Get().GetRenderer())
		{
			SlateRenderer->ReloadTextureResources();
		}
	}
}

FName UE::RenderGrid::Private::FRenderGridEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("RenderGridEditorStyle"));
	return StyleSetName;
}

const FLinearColor& UE::RenderGrid::Private::FRenderGridEditorStyle::GetColor(FName PropertyName, const ANSICHAR* Specifier)
{
	return StyleInstance->GetColor(PropertyName, Specifier);
}


const FSlateBrush* UE::RenderGrid::Private::FRenderGridEditorStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return StyleInstance->GetBrush(PropertyName, Specifier);
}

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(Style->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(Style->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BORDER_BRUSH(RelativePath, ...) FSlateBorderBrush(Style->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define IMAGE_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(Style->RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

#define CORE_IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(Style->RootToCoreContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define CORE_BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(Style->RootToCoreContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define CORE_BORDER_BRUSH(RelativePath, ...) FSlateBorderBrush(Style->RootToCoreContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define CORE_IMAGE_BRUSH_SVG(RelativePath, ...) FSlateVectorImageBrush(Style->RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

const FVector2D Icon64x64(64.f, 64.f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);

TSharedRef<FSlateStyleSet> UE::RenderGrid::Private::FRenderGridEditorStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShared<FSlateStyleSet>("RenderGridEditor");

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("RenderGrid"));
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		Style->SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	}

	Style->Set("ClassIcon.RenderGrid", new IMAGE_BRUSH_SVG("Icons/RenderGrid_16", Icon16x16));
	Style->Set("ClassIcon.RenderGridBlueprint", new IMAGE_BRUSH_SVG("Icons/RenderGrid_16", Icon16x16));
	Style->Set("ClassThumbnail.RenderGrid", new IMAGE_BRUSH_SVG("Icons/RenderGrid_64", Icon64x64));
	Style->Set("ClassThumbnail.RenderGridBlueprint", new IMAGE_BRUSH_SVG("Icons/RenderGrid_64", Icon64x64));

	Style->Set("Invisible",
		FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalPadding(FMargin())
		.SetPressedPadding(FMargin())
	);
	Style->Set("HoverHintOnly",
		FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4/16.0f), FLinearColor(1,1,1,0.05f)))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4/16.0f), FLinearColor(1,1,1,0.15f)))
		.SetNormalPadding(FMargin())
		.SetPressedPadding(FMargin())
	);

	return Style;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
#undef IMAGE_BRUSH_SVG

#undef CORE_IMAGE_BRUSH
#undef CORE_BOX_BRUSH
#undef CORE_BORDER_BRUSH
#undef CORE_IMAGE_BRUSH_SVG
