// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"


namespace UE::RenderGrid::Private
{
	/**
	 * A static class containing the style setup of the render grid plugin.
	 */
	class FRenderGridEditorStyle
	{
	public:
		static const ISlateStyle& Get();

		static void Initialize();
		static void Shutdown();
		static void ReloadTextures();

		static FName GetStyleSetName();
		static const FLinearColor& GetColor(FName PropertyName, const ANSICHAR* Specifier = nullptr);
		static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = nullptr);

		template<typename WidgetStyleType>
		static const WidgetStyleType& GetWidgetStyle(FName PropertyName, const ANSICHAR* Specifier = nullptr)
		{
			return StyleInstance->GetWidgetStyle<WidgetStyleType>(PropertyName, Specifier);
		}

	private:
		static TSharedRef<FSlateStyleSet> Create();
		static TSharedPtr<FSlateStyleSet> StyleInstance;
	};
}
