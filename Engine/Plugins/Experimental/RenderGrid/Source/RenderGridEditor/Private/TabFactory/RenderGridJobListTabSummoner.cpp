// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/RenderGridJobListTabSummoner.h"
#include "UI/Tabs/SRenderGridJobListTab.h"
#include "Styling/AppStyle.h"
#include "IRenderGridEditor.h"

#define LOCTEXT_NAMESPACE "RenderGridJobListTabSummoner"


const FName UE::RenderGrid::Private::FRenderGridJobListTabSummoner::TabID(TEXT("RenderGridJobList"));


UE::RenderGrid::Private::FRenderGridJobListTabSummoner::FRenderGridJobListTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor)
	: FWorkflowTabFactory(TabID, InBlueprintEditor)
	, BlueprintEditorWeakPtr(InBlueprintEditor)
{
	TabLabel = LOCTEXT("RenderGridJobList_TabLabel", "Jobs");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.All");
	bIsSingleton = true;
	ViewMenuDescription = LOCTEXT("RenderGridJobList_ViewMenu_Desc", "Jobs");
	ViewMenuTooltip = LOCTEXT("RenderGridJobList_ViewMenu_ToolTip", "Show the job list.");
}

TSharedRef<SWidget> UE::RenderGrid::Private::FRenderGridJobListTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRenderGridJobListTab, BlueprintEditorWeakPtr.Pin())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Job List")));
}


#undef LOCTEXT_NAMESPACE
