// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"


namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The render grid job list tab factory.
	 */
	struct FRenderGridJobListTabSummoner : FWorkflowTabFactory
	{
	public:
		/** Unique ID representing this tab. */
		static const FName TabID;

	public:
		FRenderGridJobListTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor);
		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	protected:
		/** A weak reference to the blueprint editor. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;
	};
}
