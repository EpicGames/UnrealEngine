// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/RenderGridJobPropertiesTabSummoner.h"
#include "UI/Tabs/SRenderGridJobPropertiesTab.h"
#include "Styling/AppStyle.h"
#include "IRenderGridEditor.h"

#define LOCTEXT_NAMESPACE "RenderGridJobPropertiesTabSummoner"


const FName UE::RenderGrid::Private::FRenderGridJobPropertiesTabSummoner::TabID(TEXT("RenderGridJobProperties"));


UE::RenderGrid::Private::FRenderGridJobPropertiesTabSummoner::FRenderGridJobPropertiesTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor)
	: FWorkflowTabFactory(TabID, InBlueprintEditor)
	, BlueprintEditorWeakPtr(InBlueprintEditor)
{
	TabLabel = LOCTEXT("RenderGridJobProperties_TabLabel", "Job");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon");
	bIsSingleton = true;
	ViewMenuDescription = LOCTEXT("RenderGridJobProperties_ViewMenu_Desc", "Job");
	ViewMenuTooltip = LOCTEXT("RenderGridJobProperties_ViewMenu_ToolTip", "Show the job properties.");
}

TSharedRef<SWidget> UE::RenderGrid::Private::FRenderGridJobPropertiesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRenderGridJobPropertiesTab, BlueprintEditorWeakPtr.Pin())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Job")));
}


#undef LOCTEXT_NAMESPACE
