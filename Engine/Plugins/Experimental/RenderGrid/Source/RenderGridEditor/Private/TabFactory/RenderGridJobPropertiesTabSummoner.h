// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"


namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The render grid job properties tab factory.
	 */
	struct FRenderGridJobPropertiesTabSummoner : FWorkflowTabFactory
	{
	public:
		/** Unique ID representing this tab. */
		static const FName TabID;

	public:
		FRenderGridJobPropertiesTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor);
		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	protected:
		/** A weak reference to the blueprint editor. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;
	};
}
