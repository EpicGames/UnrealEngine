// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/RenderGridPropertiesTabSummoner.h"
#include "UI/Tabs/SRenderGridPropertiesTab.h"
#include "Styling/AppStyle.h"
#include "IRenderGridEditor.h"

#define LOCTEXT_NAMESPACE "RenderGridPropertiesTabSummoner"


const FName UE::RenderGrid::Private::FRenderGridPropertiesTabSummoner::TabID(TEXT("RenderGridProperties"));


UE::RenderGrid::Private::FRenderGridPropertiesTabSummoner::FRenderGridPropertiesTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor)
	: FWorkflowTabFactory(TabID, InBlueprintEditor)
	, BlueprintEditorWeakPtr(InBlueprintEditor)
{
	TabLabel = LOCTEXT("RenderGridProperties_TabLabel", "Grid");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.ShowSourcesView");
	bIsSingleton = true;
	ViewMenuDescription = LOCTEXT("RenderGridProperties_ViewMenu_Desc", "Grid");
	ViewMenuTooltip = LOCTEXT("RenderGridProperties_ViewMenu_ToolTip", "Show the grid properties.");
}

TSharedRef<SWidget> UE::RenderGrid::Private::FRenderGridPropertiesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRenderGridPropertiesTab, BlueprintEditorWeakPtr.Pin())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Grid")));
}


#undef LOCTEXT_NAMESPACE
