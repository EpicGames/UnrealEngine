// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabFactory/RenderGridViewerTabSummoner.h"
#include "UI/Tabs/SRenderGridViewerTab.h"
#include "Styling/AppStyle.h"
#include "IRenderGridEditor.h"

#define LOCTEXT_NAMESPACE "RenderGridViewerTabSummoner"


const FName UE::RenderGrid::Private::FRenderGridViewerTabSummoner::TabID(TEXT("RenderGridViewer"));


UE::RenderGrid::Private::FRenderGridViewerTabSummoner::FRenderGridViewerTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor)
	: FWorkflowTabFactory(TabID, InBlueprintEditor)
	, BlueprintEditorWeakPtr(InBlueprintEditor)
{
	TabLabel = LOCTEXT("RenderGridViewer_TabLabel", "Viewer");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");
	bIsSingleton = true;
	ViewMenuDescription = LOCTEXT("RenderGridViewer_ViewMenu_Desc", "Viewer");
	ViewMenuTooltip = LOCTEXT("RenderGridViewer_ViewMenu_ToolTip", "Show the viewer.");
}

TSharedRef<SWidget> UE::RenderGrid::Private::FRenderGridViewerTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRenderGridViewerTab, BlueprintEditorWeakPtr.Pin())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Viewer")));
}


#undef LOCTEXT_NAMESPACE
