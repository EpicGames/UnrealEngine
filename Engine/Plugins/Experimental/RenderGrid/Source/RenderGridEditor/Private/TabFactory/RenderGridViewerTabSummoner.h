// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"


namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The render grid viewer tab factory.
	 */
	struct FRenderGridViewerTabSummoner : FWorkflowTabFactory
	{
	public:
		/** Unique ID representing this tab. */
		static const FName TabID;

	public:
		FRenderGridViewerTabSummoner(TSharedPtr<IRenderGridEditor> InBlueprintEditor);
		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	protected:
		/** A weak reference to the blueprint editor. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;
	};
}
