// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkit/RenderGridEditor.h"

#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridManager.h"
#include "RenderGrid/RenderGridQueue.h"
#include "RenderGridEditorToolbar.h"
#include "IRenderGridEditorModule.h"

#include "BlueprintCompilationManager.h"
#include "BlueprintModes/RenderGridApplicationModeBase.h"
#include "BlueprintModes/RenderGridApplicationModeListing.h"
#include "BlueprintModes/RenderGridApplicationModeLogic.h"
#include "BlueprintModes/RenderGridApplicationModes.h"
#include "Blueprints/RenderGridBlueprint.h"
#include "Commands/RenderGridEditorCommands.h"
#include "EdGraphNode_Comment.h"
#include "EditorModeManager.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyCustomizationHelpers.h"
#include "SBlueprintEditorToolbar.h"
#include "ScopedTransaction.h"
#include "SGraphPanel.h"
#include "SKismetInspector.h"
#include "SMyBlueprint.h"
#include "Stats/StatsHierarchical.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FRenderGridEditor"


namespace UE::RenderGrid::Private
{
	const FName RenderGridEditorAppName(TEXT("RenderGridEditorApp"));
}


URenderGridJobSelection::URenderGridJobSelection()
{
	SetFlags(RF_Public | RF_Transactional);
}

bool URenderGridJobSelection::SetSelectedRenderGridJobs(const TArray<URenderGridJob*>& Jobs)
{
	TSet<FGuid> PreviouslySelectedJobIds;
	PreviouslySelectedJobIds.Append(SelectedRenderGridJobIds);

	SelectedRenderGridJobIds.Empty();
	for (URenderGridJob* Job : Jobs)
	{
		if (!IsValid(Job))
		{
			continue;
		}
		SelectedRenderGridJobIds.Add(Job->GetGuid());
	}
	if (SelectedRenderGridJobIds.Num() != PreviouslySelectedJobIds.Num())
	{
		return true;
	}
	for (const FGuid& JobId : SelectedRenderGridJobIds)
	{
		if (!PreviouslySelectedJobIds.Contains(JobId))
		{
			return true;
		}
	}
	return false;
}


TStrongObjectPtr<URenderGridJobSelection> UE::RenderGrid::Private::FRenderGridEditor::RenderGridJobSelection = TStrongObjectPtr(NewObject<URenderGridJobSelection>());

void UE::RenderGrid::Private::FRenderGridEditor::InitRenderGridEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, URenderGridBlueprint* InRenderGridBlueprint)
{
	check(InRenderGridBlueprint);

	FBlueprintCompilationManager::FlushCompilationQueue(nullptr);

	TSharedPtr<IRenderGridEditor> ThisPtr(SharedThis(this));

	if (!Toolbar.IsValid())
	{
		Toolbar = MakeShareable(new FBlueprintEditorToolbar(SharedThis(this)));
	}

	RenderGridToolbar = MakeShared<FRenderGridBlueprintEditorToolbar>(ThisPtr);

	// Build up a list of objects being edited in this asset editor
	TArray<UObject*> ObjectsBeingEdited;
	ObjectsBeingEdited.Add(InRenderGridBlueprint);

	// Initialize the asset editor and spawn tabs
	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	InitAssetEditor(Mode, InitToolkitHost, RenderGridEditorAppName, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsBeingEdited);

	CreateDefaultCommands();

	TArray<UBlueprint*> RenderGridBlueprints;
	RenderGridBlueprints.Add(InRenderGridBlueprint);

	CommonInitialization(RenderGridBlueprints, false);

	BindCommands();

	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	UpdateInstance(GetRenderGridBlueprint(), true);
	if (URenderGridBlueprint* RenderGridBlueprint = GetRenderGridBlueprint())
	{
		RenderGridBlueprint->PropagateAllPropertiesToInstances();
	}
	constexpr bool bShouldOpenInDefaultsMode = true;
	RegisterApplicationModes(RenderGridBlueprints, bShouldOpenInDefaultsMode, InRenderGridBlueprint->bIsNewlyCreated);

	// Post-layout initialization
	PostLayoutBlueprintEditorInitialization();
}

UE::RenderGrid::Private::FRenderGridEditor::FRenderGridEditor()
	: PreviewBlueprintWeakPtr(nullptr)
	, bRunRenderNewBatch(false)
	, BatchRenderQueue(nullptr)
	, PreviewRenderQueue(nullptr)
	, DebuggingTimeInSecondsRemaining(0)
	, bIsDebugging(false)
	, bPreviousShouldHideUI(false)
{}

UE::RenderGrid::Private::FRenderGridEditor::~FRenderGridEditor()
{
	DestroyInstance();
}

void UE::RenderGrid::Private::FRenderGridEditor::CreateDefaultCommands()
{
	if (GetBlueprintObj())
	{
		FBlueprintEditor::CreateDefaultCommands();
	}
	else
	{
		ToolkitCommands->MapAction(FGenericCommands::Get().Undo, FExecuteAction::CreateSP(this, &FRenderGridEditor::UndoAction));
		ToolkitCommands->MapAction(FGenericCommands::Get().Redo, FExecuteAction::CreateSP(this, &FRenderGridEditor::RedoAction));
	}
}

UBlueprint* UE::RenderGrid::Private::FRenderGridEditor::GetBlueprintObj() const
{
	for (UObject* Obj : GetEditingObjects())
	{
		if (!IsValid(Obj))
		{
			continue;
		}
		if (UBlueprint* Result = Cast<UBlueprint>(Obj))
		{
			return Result;
		}
	}
	return nullptr;
}

FGraphAppearanceInfo UE::RenderGrid::Private::FRenderGridEditor::GetGraphAppearance(UEdGraph* InGraph) const
{
	FGraphAppearanceInfo AppearanceInfo = FBlueprintEditor::GetGraphAppearance(InGraph);
	if (GetBlueprintObj()->IsA(URenderGridBlueprint::StaticClass()))
	{
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_RenderGrid", "RENDER GRID");
	}
	return AppearanceInfo;
}

void UE::RenderGrid::Private::FRenderGridEditor::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
	if (!NewlyActivated.IsValid())
	{
		TArray<UObject*> ObjArray;
		Inspector->ShowDetailsForObjects(ObjArray);
	}
	else
	{
		FBlueprintEditor::OnActiveTabChanged(PreviouslyActive, NewlyActivated);
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents)
{
	FBlueprintEditor::SetupGraphEditorEvents(InGraph, InEvents);

	InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FRenderGridEditor::HandleCreateGraphActionMenu);
}

void UE::RenderGrid::Private::FRenderGridEditor::RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, bool bNewlyCreated)
{
	if (InBlueprints.Num() == 1)
	{
		TSharedPtr<FRenderGridEditor> ThisPtr(SharedThis(this));

		// Create the modes and activate one (which will populate with a real layout)
		TArray<TSharedRef<FApplicationMode>> TempModeList;
		TempModeList.Add(MakeShareable(new FRenderGridApplicationModeListing(ThisPtr)));
		TempModeList.Add(MakeShareable(new FRenderGridApplicationModeLogic(ThisPtr)));

		for (TSharedRef<FApplicationMode>& AppMode : TempModeList)
		{
			AddApplicationMode(AppMode->GetModeName(), AppMode);
		}

		SetCurrentMode(FRenderGridApplicationModes::ListingMode);

		// Activate our edit mode
		GetEditorModeManager().SetDefaultMode(FRenderGridApplicationModes::ListingMode);
		GetEditorModeManager().ActivateMode(FRenderGridApplicationModes::ListingMode);
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::PostUndo(bool bSuccessful)
{
	FBlueprintEditor::PostUndo(bSuccessful);

	if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint))
	{
		PreviewBlueprint->PropagateAllPropertiesToInstances();
	}

	OnRenderGridChanged().Broadcast();
	OnRenderGridJobsSelectionChanged().Broadcast();
}

void UE::RenderGrid::Private::FRenderGridEditor::PostRedo(bool bSuccessful)
{
	FBlueprintEditor::PostRedo(bSuccessful);

	if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint))
	{
		PreviewBlueprint->PropagateAllPropertiesToInstances();
	}

	OnRenderGridChanged().Broadcast();
	OnRenderGridJobsSelectionChanged().Broadcast();
}

void UE::RenderGrid::Private::FRenderGridEditor::Compile()
{
	DestroyInstance();
	FBlueprintEditor::Compile();
}

URenderGridBlueprint* UE::RenderGrid::Private::FRenderGridEditor::GetRenderGridBlueprint() const
{
	return Cast<URenderGridBlueprint>(GetBlueprintObj());
}

URenderGrid* UE::RenderGrid::Private::FRenderGridEditor::GetInstance() const
{
	return RenderGridWeakPtr.Get();
}

void UE::RenderGrid::Private::FRenderGridEditor::SetIsDebugging(const bool bInIsDebugging)
{
	if (bInIsDebugging)
	{
		bIsDebugging = true;
		DebuggingTimeInSecondsRemaining = TimeInSecondsToRemainDebugging;
		GetBlueprintObj()->SetObjectBeingDebugged(GetInstance());
	}
	else
	{
		bIsDebugging = false;
	}
}

bool UE::RenderGrid::Private::FRenderGridEditor::IsBatchRendering() const
{
	if (URenderGridQueue* Queue = BatchRenderQueue.Get(); IsValid(Queue))
	{
		return Queue->IsCurrentlyRendering();
	}
	return false;
}

bool UE::RenderGrid::Private::FRenderGridEditor::IsPreviewRendering() const
{
	if (URenderGridQueue* Queue = PreviewRenderQueue.Get(); IsValid(Queue))
	{
		return Queue->IsCurrentlyRendering();
	}
	return false;
}

void UE::RenderGrid::Private::FRenderGridEditor::SetPreviewRenderQueue(URenderGridQueue* Queue)
{
	PreviewRenderQueue = TStrongObjectPtr(Queue);
	SetIsDebugging(IsValid(Queue));
}

void UE::RenderGrid::Private::FRenderGridEditor::MarkAsModified()
{
	if (URenderGridBlueprint* BlueprintObj = Cast<URenderGridBlueprint>(GetBlueprintObj()); IsValid(BlueprintObj))
	{
		BlueprintObj->Modify();
		BlueprintObj->GetRenderGrid()->Modify();
	}
}

TArray<URenderGridJob*> UE::RenderGrid::Private::FRenderGridEditor::GetSelectedRenderGridJobs() const
{
	TArray<URenderGridJob*> SelectedJobs;
	if (ShouldHideUI())
	{
		return SelectedJobs;
	}

	if (URenderGrid* Grid = GetInstance(); IsValid(Grid))
	{
		if (URenderGridJobSelection* JobSelection = RenderGridJobSelection.Get(); IsValid(JobSelection))
		{
			for (URenderGridJob* Job : Grid->GetRenderGridJobs())
			{
				if (JobSelection->SelectedRenderGridJobIds.Contains(Job->GetGuid()))
				{
					SelectedJobs.Add(Job);
				}
			}
		}
	}
	return SelectedJobs;
}

void UE::RenderGrid::Private::FRenderGridEditor::SetSelectedRenderGridJobs(const TArray<URenderGridJob*>& Jobs)
{
	if (URenderGridJobSelection* JobSelection = RenderGridJobSelection.Get(); IsValid(JobSelection))
	{
		JobSelection->Modify();
		JobSelection->SetSelectedRenderGridJobs(Jobs);
	}
}

FName UE::RenderGrid::Private::FRenderGridEditor::GetToolkitFName() const
{
	return FName("RenderGridEditor");
}

FText UE::RenderGrid::Private::FRenderGridEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Render Grid Editor");
}

FString UE::RenderGrid::Private::FRenderGridEditor::GetDocumentationLink() const
{
	return FString();
}

FText UE::RenderGrid::Private::FRenderGridEditor::GetToolkitToolTipText() const
{
	return GetToolTipTextForObject(GetBlueprintObj());
}

FLinearColor UE::RenderGrid::Private::FRenderGridEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.5f, 0.25f, 0.35f, 0.5f);
}

FString UE::RenderGrid::Private::FRenderGridEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Render Grid ").ToString();
}

void UE::RenderGrid::Private::FRenderGridEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBlueprintEditor::InitToolMenuContext(MenuContext);
}

void UE::RenderGrid::Private::FRenderGridEditor::Tick(float DeltaTime)
{
	FBlueprintEditor::Tick(DeltaTime);

	// Note: The weak ptr can become stale if the actor is reinstanced due to a Blueprint change, etc. In that case we 
	//       look to see if we can find the new instance in the preview world and then update the weak ptr.
	if (RenderGridWeakPtr.IsStale(true))
	{
		RefreshInstance();
	}

	if (bRunRenderNewBatch)
	{
		bRunRenderNewBatch = false;
		BatchRenderListAction();
	}

	if (ShouldHideUI() != bPreviousShouldHideUI)
	{
		bPreviousShouldHideUI = !bPreviousShouldHideUI;
		OnRenderGridShouldHideUIChanged().Broadcast();
	}

	if (URenderGridJobSelection* JobSelection = RenderGridJobSelection.Get(); IsValid(JobSelection))
	{
		if (bPreviousShouldHideUI && !PreviousSelectedRenderGridJobIds.IsEmpty())
		{
			PreviousSelectedRenderGridJobIds.Empty();
			OnRenderGridJobsSelectionChanged().Broadcast();
		}
		else
		{
			if ((JobSelection->SelectedRenderGridJobIds.Num() != PreviousSelectedRenderGridJobIds.Num()) || (JobSelection->SelectedRenderGridJobIds.Intersect(PreviousSelectedRenderGridJobIds).Num() != PreviousSelectedRenderGridJobIds.Num()))
			{
				PreviousSelectedRenderGridJobIds = JobSelection->SelectedRenderGridJobIds;
				OnRenderGridJobsSelectionChanged().Broadcast();
			}
		}
	}

	if ((DebuggingTimeInSecondsRemaining > 0) && !bIsDebugging)
	{
		DebuggingTimeInSecondsRemaining -= DeltaTime;
		if (DebuggingTimeInSecondsRemaining <= 0)
		{
			GetBlueprintObj()->SetObjectBeingDebugged(nullptr);
		}
	}

	if (URenderGrid* Instance = GetInstance(); IsValid(Instance))
	{
		Instance->Tick(DeltaTime);
	}
}

TStatId UE::RenderGrid::Private::FRenderGridEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FRenderGridEditor, STATGROUP_Tickables);
}

void UE::RenderGrid::Private::FRenderGridEditor::RefreshInstance()
{
	UpdateInstance(GetRenderGridBlueprint(), true);
}

void UE::RenderGrid::Private::FRenderGridEditor::OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled)
{
	DestroyInstance();

	FBlueprintEditor::OnBlueprintChangedImpl(InBlueprint, bIsJustBeingCompiled);

	if (InBlueprint)
	{
		RefreshInstance();
		if (URenderGridBlueprint* RenderGridBlueprint = Cast<URenderGridBlueprint>(InBlueprint); IsValid(RenderGridBlueprint))
		{
			RenderGridBlueprint->PropagateAllPropertiesToInstances();
		}
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	FBlueprintEditor::AddReferencedObjects(Collector);

	if (URenderGrid* Instance = RenderGridWeakPtr.Get(); IsValid(Instance))
	{
		Collector.AddReferencedObject(RenderGridWeakPtr);
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	FBlueprintEditor::NotifyPreChange(PropertyAboutToChange);

	if (URenderGridBlueprint* RenderGridBlueprint = GetRenderGridBlueprint())
	{
		RenderGridBlueprint->Modify();
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	FBlueprintEditor::NotifyPostChange(PropertyChangedEvent, PropertyThatChanged);
}

void UE::RenderGrid::Private::FRenderGridEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) {}

void UE::RenderGrid::Private::FRenderGridEditor::BindCommands()
{
	const FRenderGridEditorCommands& Commands = FRenderGridEditorCommands::Get();

	ToolkitCommands->MapAction(Commands.AddJob, FExecuteAction::CreateSP(this, &FRenderGridEditor::AddJobAction));
	ToolkitCommands->MapAction(Commands.DuplicateJob, FExecuteAction::CreateSP(this, &FRenderGridEditor::DuplicateJobAction));
	ToolkitCommands->MapAction(Commands.DeleteJob, FExecuteAction::CreateSP(this, &FRenderGridEditor::DeleteJobAction));
	ToolkitCommands->MapAction(Commands.BatchRenderList, FExecuteAction::CreateSP(this, &FRenderGridEditor::BatchRenderListAction));
}

void UE::RenderGrid::Private::FRenderGridEditor::AddJobAction()
{
	if (URenderGrid* Instance = GetInstance(); IsValid(Instance))
	{
		FScopedTransaction Transaction(LOCTEXT("AddJob", "Add Job"));
		MarkAsModified();

		if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint))
		{
			if (URenderGridJob* Job = PreviewBlueprint->GetRenderGrid()->CreateAndAddNewRenderGridJob(); IsValid(Job))
			{
				PreviewBlueprint->PropagateJobsToInstances();
				OnRenderGridJobCreated().Broadcast(Job);
				OnRenderGridChanged().Broadcast();
			}
		}
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::DuplicateJobAction()
{
	TArray<URenderGridJob*> SelectedJobs = GetSelectedRenderGridJobs();
	if (SelectedJobs.Num() <= 0)
	{
		return;
	}

	if (URenderGrid* Instance = GetInstance(); IsValid(Instance))
	{
		FScopedTransaction Transaction(LOCTEXT("DuplicateJob", "Duplicate Job"));
		MarkAsModified();

		if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint))
		{
			for (URenderGridJob* SelectedJob : SelectedJobs)
			{
				PreviewBlueprint->GetRenderGrid()->DuplicateAndAddRenderGridJob(SelectedJob);
			}
			PreviewBlueprint->PropagateJobsToInstances();
			OnRenderGridChanged().Broadcast();
		}
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::DeleteJobAction()
{
	TArray<URenderGridJob*> SelectedJobs = GetSelectedRenderGridJobs();
	if (SelectedJobs.Num() <= 0)
	{
		return;
	}

	if (URenderGrid* Instance = GetInstance(); IsValid(Instance))
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteJob", "Delete Job"));
		MarkAsModified();

		if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint))
		{
			for (URenderGridJob* SelectedJob : SelectedJobs)
			{
				PreviewBlueprint->GetRenderGrid()->RemoveRenderGridJob(SelectedJob);
			}
			PreviewBlueprint->PropagateJobsToInstances();
			OnRenderGridChanged().Broadcast();
		}
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::BatchRenderListAction()
{
	if (!CanCurrentlyRender())
	{
		bRunRenderNewBatch = true;
		return;
	}

	if (URenderGrid* Instance = GetInstance(); IsValid(Instance))
	{
		if (URenderGridQueue* RenderQueue = Instance->Render(); IsValid(RenderQueue))
		{
			BatchRenderQueue = TStrongObjectPtr(RenderQueue);

			TWeakPtr<FRenderGridEditor> ThisWeakPtr = SharedThis(this);
			RenderQueue->OnExecuteFinished().AddLambda([ThisWeakPtr](URenderGridQueue* Queue, bool bSuccess)
			{
				if (const TSharedPtr<FRenderGridEditor> This = ThisWeakPtr.Pin())
				{
					This->OnBatchRenderListActionFinished();
				}
			});

			SetIsDebugging(true);
		}
	}
}

void UE::RenderGrid::Private::FRenderGridEditor::OnBatchRenderListActionFinished()
{
	BatchRenderQueue = nullptr;
	SetIsDebugging(false);
}

void UE::RenderGrid::Private::FRenderGridEditor::UndoAction()
{
	GEditor->UndoTransaction();
}

void UE::RenderGrid::Private::FRenderGridEditor::RedoAction()
{
	GEditor->RedoTransaction();
}

void UE::RenderGrid::Private::FRenderGridEditor::ExtendMenu()
{
	if (MenuExtender.IsValid())
	{
		RemoveMenuExtender(MenuExtender);
		MenuExtender.Reset();
	}

	MenuExtender = MakeShareable(new FExtender);

	AddMenuExtender(MenuExtender);

	// Add extensible menu if exists
	IRenderGridEditorModule& RenderGridEditorModule = IRenderGridEditorModule::Get();
	AddMenuExtender(RenderGridEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void UE::RenderGrid::Private::FRenderGridEditor::ExtendToolbar()
{
	// If the ToolbarExtender is valid, remove it before rebuilding it
	if (ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	IRenderGridEditorModule& RenderGridEditorModule = IRenderGridEditorModule::Get();
	AddToolbarExtender(RenderGridEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FRenderGridEditor::FillToolbar)
	);
}

void UE::RenderGrid::Private::FRenderGridEditor::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("Common");
	{
		ToolbarBuilder.AddToolBarButton(FRenderGridEditorCommands::Get().BatchRenderList, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.OpenCinematic"));
	}
	ToolbarBuilder.EndSection();
}

void UE::RenderGrid::Private::FRenderGridEditor::DestroyInstance()
{
	if (URenderGrid* Instance = RenderGridWeakPtr.Get(); IsValid(Instance))
	{
		// Execute the blueprint event
		Instance->EndEditor();
	}

	if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint))
	{
		PreviewBlueprint->Save();
	}

	RenderGridWeakPtr.Reset();
	PreviewBlueprintWeakPtr.Reset();
}

void UE::RenderGrid::Private::FRenderGridEditor::UpdateInstance(UBlueprint* InBlueprint, bool bInForceFullUpdate)
{
	// If the Blueprint is changing
	if ((InBlueprint != PreviewBlueprintWeakPtr.Get()) || bInForceFullUpdate)
	{
		// Destroy the previous instance
		DestroyInstance();

		// Save the Blueprint we're creating a preview for
		PreviewBlueprintWeakPtr = Cast<URenderGridBlueprint>(InBlueprint);

		URenderGrid* RenderGrid = nullptr;
		if (URenderGridBlueprint* PreviewBlueprint = PreviewBlueprintWeakPtr.Get(); IsValid(PreviewBlueprint) && IsValid(PreviewBlueprint->GeneratedClass))
		{
			PreviewBlueprint->Load();
			FMakeClassSpawnableOnScope TemporarilySpawnable(PreviewBlueprint->GeneratedClass);
			RenderGrid = NewObject<URenderGrid>(PreviewScene.GetWorld(), PreviewBlueprint->GeneratedClass);
		}

		// Set the debug object again, if it was debugging
		GetBlueprintObj()->SetObjectBeingDebugged(bIsDebugging ? RenderGrid : nullptr);

		// Store a reference to the preview actor.
		RenderGridWeakPtr = RenderGrid;

		// Broadcast the events
		OnRenderGridChanged().Broadcast();
		OnRenderGridJobsSelectionChanged().Broadcast();

		// Execute the blueprint event
		if (IsValid(RenderGrid))
		{
			RenderGrid->BeginEditor();
		}
	}
}

FActionMenuContent UE::RenderGrid::Private::FRenderGridEditor::HandleCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	return OnCreateGraphActionMenu(InGraph, InNodePosition, InDraggedPins, bAutoExpand, InOnMenuClosed);
}


#undef LOCTEXT_NAMESPACE
