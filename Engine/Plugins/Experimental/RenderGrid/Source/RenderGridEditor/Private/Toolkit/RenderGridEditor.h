// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRenderGridEditor.h"
#include "RenderGridEditor.generated.h"


class FSpawnTabArgs;
class IToolkitHost;
class SDockTab;
class URenderGrid;
class URenderGridJob;
class URenderGridBlueprint;
class URenderGridQueue;

namespace UE::RenderGrid::Private
{
	class FRenderGridBlueprintEditorToolbar;
}


/**
 * The selected render grid jobs.
 * This is placed in a separate class, so it can be an UObject, so it can be placed in the undo/redo buffer.
 */
UCLASS()
class RENDERGRIDEDITOR_API URenderGridJobSelection : public UObject
{
	GENERATED_BODY()

public:
	URenderGridJobSelection();

	/** Sets the new job selection. Returns true if it is different from the previous selection, returns false if it was exactly the same. */
	bool SetSelectedRenderGridJobs(const TArray<URenderGridJob*>& Jobs);

public:
	/** The GUIDs of the currently selected render grid jobs. */
	UPROPERTY()
	TSet<FGuid> SelectedRenderGridJobIds;
};


namespace UE::RenderGrid::Private
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FRenderGridEditorClosed, const UE::RenderGrid::IRenderGridEditor*, URenderGridBlueprint*);


	/**
	 * The render grid editor implementation.
	 */
	class FRenderGridEditor : public IRenderGridEditor
	{
	public:
		/** The time it should remain in debugging mode after it has been turned off. */
		static constexpr float TimeInSecondsToRemainDebugging = 4.0f;

	protected:
		/** The currently selected render grid jobs. It's static so multiple open render grid editors won't cause bugs with each other. */
		static TStrongObjectPtr<URenderGridJobSelection> RenderGridJobSelection;


	public:
		void InitRenderGridEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, URenderGridBlueprint* InRenderGridBlueprint);

	public:
		FRenderGridEditor();
		virtual ~FRenderGridEditor() override;

		//~ Begin FBlueprintEditor Interface
		virtual void CreateDefaultCommands() override;
		virtual UBlueprint* GetBlueprintObj() const override;
		virtual FGraphAppearanceInfo GetGraphAppearance(UEdGraph* InGraph) const override;
		virtual bool IsInAScriptingMode() const override { return true; }
		virtual void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated) override;
		virtual void SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents) override;
		virtual void RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, bool bNewlyCreated = false) override;
		virtual void PostUndo(bool bSuccessful) override;
		virtual void PostRedo(bool bSuccessful) override;
		virtual void Compile() override;
		//~ End FBlueprintEditor Interface

		//~ Begin IRenderGridEditor Interface
		virtual URenderGridBlueprint* GetRenderGridBlueprint() const override;
		virtual URenderGrid* GetInstance() const override;
		virtual void SetIsDebugging(const bool bInIsDebugging) override;
		virtual TSharedPtr<FRenderGridBlueprintEditorToolbar> GetRenderGridToolbarBuilder() override { return RenderGridToolbar; }
		virtual bool IsBatchRendering() const override;
		virtual URenderGridQueue* GetBatchRenderQueue() const override { return BatchRenderQueue.Get(); }
		virtual bool IsPreviewRendering() const override;
		virtual URenderGridQueue* GetPreviewRenderQueue() const override { return PreviewRenderQueue.Get(); }
		virtual void SetPreviewRenderQueue(URenderGridQueue* Queue) override;
		virtual void MarkAsModified() override;
		virtual TArray<URenderGridJob*> GetSelectedRenderGridJobs() const override;
		virtual void SetSelectedRenderGridJobs(const TArray<URenderGridJob*>& Jobs) override;
		//~ End IRenderGridEditor Interface

		//~ Begin IToolkit Interface
		virtual FName GetToolkitFName() const override;
		virtual FText GetBaseToolkitName() const override;
		virtual FString GetDocumentationLink() const override;
		virtual FText GetToolkitToolTipText() const override;
		virtual FLinearColor GetWorldCentricTabColorScale() const override;
		virtual FString GetWorldCentricTabPrefix() const override;
		virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
		//~ End IToolkit Interface

		//~ Begin FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		//~ End FTickableEditorObject Interface

		/** Immediately rebuilds the render grid instance that is being shown in the editor. */
		void RefreshInstance();

	private:
		/** Called whenever the blueprint is structurally changed. */
		virtual void OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled = false) override;

	protected:
		//~ Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		//~ End FGCObject Interface

		//~ Begin FNotifyHook Interface
		virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
		virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override;
		//~ End FNotifyHook Interface

	protected:
		/** Binds the FRenderGridEditorCommands commands to functions in this editor. */
		void BindCommands();

		/** Creates and adds a new render grid job. */
		void AddJobAction();

		/** Duplicates the selected render grid job(s). */
		void DuplicateJobAction();

		/** Removes the currently selected render grid job(s). */
		void DeleteJobAction();

		/** Renders all the currently enabled render grid jobs. */
		void BatchRenderListAction();

		/** The callback for when the batch render list action finishes. */
		void OnBatchRenderListActionFinished();

		/** Undo the last action. */
		void UndoAction();

		/** Redo the last action that was undone. */
		void RedoAction();

	private:
		/** Extends the menu. */
		void ExtendMenu();

		/** Extends the toolbar. */
		void ExtendToolbar();

		/** Fills the toolbar with content. */
		void FillToolbar(FToolBarBuilder& ToolbarBuilder);

		/** Destroy the render grid instance that is currently visible in the editor. */
		void DestroyInstance();

		/** Makes a newly compiled/opened render grid instance visible in the editor. */
		void UpdateInstance(UBlueprint* InBlueprint, bool bInForceFullUpdate);

		/** Wraps the normal blueprint editor's action menu creation callback. */
		FActionMenuContent HandleCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	private:
		/** The toolbar builder that is used to customize the toolbar of this editor. */
		TSharedPtr<FRenderGridBlueprintEditorToolbar> RenderGridToolbar;

	protected:
		/** The extender to pass to the level editor to extend it's window menu. */
		TSharedPtr<FExtender> MenuExtender;

		/** The toolbar extender of this editor. */
		TSharedPtr<FExtender> ToolbarExtender;

		/** The blueprint instance that's currently visible in the editor. */
		TWeakObjectPtr<URenderGridBlueprint> PreviewBlueprintWeakPtr;

		/** The current render grid instance that's visible in the editor. */
		mutable TWeakObjectPtr<URenderGrid> RenderGridWeakPtr;

		/** True if it should call BatchRenderListAction() next frame. */
		bool bRunRenderNewBatch;

		/** The current batch render queue, if any. */
		TStrongObjectPtr<URenderGridQueue> BatchRenderQueue;

		/** The current preview render queue, if any. */
		TStrongObjectPtr<URenderGridQueue> PreviewRenderQueue;

		/** The time it should still remain in debugging mode (after it has been turned off). */
		float DebuggingTimeInSecondsRemaining;

		/** True if the graph editor is currently in debugging mode. */
		bool bIsDebugging;

		/** The value of ShouldHideUI() during the last tick. */
		bool bPreviousShouldHideUI;

		/** The value of RenderGridJobSelection.Get()->SelectedRenderGridJobIds() during the last tick. */
		TSet<FGuid> PreviousSelectedRenderGridJobIds;
	};
}
