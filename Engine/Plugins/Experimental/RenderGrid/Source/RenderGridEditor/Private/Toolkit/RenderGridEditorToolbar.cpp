// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkit/RenderGridEditorToolbar.h"
#include "BlueprintModes/RenderGridApplicationModes.h"
#include "Commands/RenderGridEditorCommands.h"
#include "IRenderGridEditor.h"
#include "Styles/RenderGridEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDocumentation.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SToolTip.h"
#include "WorkflowOrientedApp/SModeWidget.h"

#define LOCTEXT_NAMESPACE "RenderGridEditorToolbar"


//////////////////////////////////////////////////////////////////////////
// SBlueprintModeSeparator

class SBlueprintModeSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SBlueprintModeSeparator) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
		);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		constexpr float Height = 20.0f;
		constexpr float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};

//////////////////////////////////////////////////////////////////////////
// FRenderGridBlueprintEditorToolbar

UE::RenderGrid::Private::FRenderGridBlueprintEditorToolbar::FRenderGridBlueprintEditorToolbar(TSharedPtr<IRenderGridEditor>& InRenderGridEditor)
	: BlueprintEditorWeakPtr(InRenderGridEditor)
{}

void UE::RenderGrid::Private::FRenderGridBlueprintEditorToolbar::AddRenderGridBlueprintEditorModesToolbar(TSharedPtr<FExtender> Extender)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		Extender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			BlueprintEditor->GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateSP(this, &FRenderGridBlueprintEditorToolbar::FillRenderGridBlueprintEditorModesToolbar));
	}
}

void UE::RenderGrid::Private::FRenderGridBlueprintEditorToolbar::AddListingModeToolbar(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Tools");

	Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		FRenderGridEditorCommands::Get().AddJob,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
	));

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		FRenderGridEditorCommands::Get().DuplicateJob,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate")
	));

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		FRenderGridEditorCommands::Get().DeleteJob,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus")
	));
}

void UE::RenderGrid::Private::FRenderGridBlueprintEditorToolbar::AddLogicModeToolbar(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Tools");

	Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);
}

void UE::RenderGrid::Private::FRenderGridBlueprintEditorToolbar::FillRenderGridBlueprintEditorModesToolbar(FToolBarBuilder& ToolbarBuilder)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		UBlueprint* BlueprintObj = BlueprintEditor->GetBlueprintObj();
		if (!BlueprintObj || (!FBlueprintEditorUtils::IsLevelScriptBlueprint(BlueprintObj) && !FBlueprintEditorUtils::IsInterfaceBlueprint(BlueprintObj) && !BlueprintObj->bIsNewlyCreated))
		{
			TAttribute<FName> GetActiveMode(BlueprintEditor.ToSharedRef(), &FBlueprintEditor::GetCurrentMode);
			FOnModeChangeRequested SetActiveMode = FOnModeChangeRequested::CreateSP(BlueprintEditor.ToSharedRef(), &FBlueprintEditor::SetCurrentMode);

			// Left side padding
			BlueprintEditor->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(4.0f, 1.0f)));

			BlueprintEditor->AddToolbarWidget(
				SNew(SModeWidget, FRenderGridApplicationModes::GetLocalizedMode(FRenderGridApplicationModes::ListingMode), FRenderGridApplicationModes::ListingMode)
				.OnGetActiveMode(GetActiveMode)
				.OnSetActiveMode(SetActiveMode)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
					LOCTEXT("ListingModeButtonTooltip", "Switch to Job Listing Mode"),
					nullptr,
					TEXT("Shared/Editors/BlueprintEditor"),
					TEXT("ListingMode")))
				.IconImage(FAppStyle::GetBrush("BTEditor.Graph.NewTask"))
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ListingMode")))
			);

			BlueprintEditor->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));

			BlueprintEditor->AddToolbarWidget(
				SNew(SModeWidget, FRenderGridApplicationModes::GetLocalizedMode(FRenderGridApplicationModes::LogicMode), FRenderGridApplicationModes::LogicMode)
				.OnGetActiveMode(GetActiveMode)
				.OnSetActiveMode(SetActiveMode)
				.CanBeSelected(BlueprintEditor.Get(), &FBlueprintEditor::IsEditingSingleBlueprint)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
					LOCTEXT("LogicModeButtonTooltip", "Switch to Logic Editing Mode"),
					nullptr,
					TEXT("Shared/Editors/BlueprintEditor"),
					TEXT("GraphMode")))
				.IconImage(FAppStyle::GetBrush("Icons.Blueprint"))
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("LogicMode")))
			);

			// Right side padding
			BlueprintEditor->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(10.0f, 1.0f)));
		}
	}
}


#undef LOCTEXT_NAMESPACE
