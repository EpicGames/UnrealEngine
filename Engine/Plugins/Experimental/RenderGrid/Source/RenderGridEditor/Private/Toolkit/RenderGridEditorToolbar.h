// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class FExtender;
class FToolBarBuilder;
class UToolMenu;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * Handles all of the toolbar related construction for the render grid blueprint editor.
	 */
	class FRenderGridBlueprintEditorToolbar : public TSharedFromThis<FRenderGridBlueprintEditorToolbar>
	{
	public:
		FRenderGridBlueprintEditorToolbar(TSharedPtr<IRenderGridEditor>& InRenderGridEditor);

		/** Adds the mode-switch UI to the editor. */
		void AddRenderGridBlueprintEditorModesToolbar(TSharedPtr<FExtender> Extender);

		/** Adds the toolbar for the listing mode to the editor. */
		void AddListingModeToolbar(UToolMenu* InMenu);

		/** Adds the toolbar for the logic mode to the editor. */
		void AddLogicModeToolbar(UToolMenu* InMenu);

	public:
		/** Creates the mode-switch UI. */
		void FillRenderGridBlueprintEditorModesToolbar(FToolBarBuilder& ToolbarBuilder);
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;
	};
}
