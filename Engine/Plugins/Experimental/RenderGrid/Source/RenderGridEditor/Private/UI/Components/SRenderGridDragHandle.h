// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Styling/CoreStyle.h"
#include "Input/DragAndDrop.h"


/**
 * A reusable widget for drag and drop functionality.
 */
template<typename DragDropType>
class SRenderGridDragHandle : public SCompoundWidget
{
public:
	using DragDropWidgetType = typename DragDropType::WidgetType;
	using HeldItemType = typename DragDropType::HeldItemType;

	SLATE_BEGIN_ARGS(SRenderGridDragHandle) {}
		SLATE_ARGUMENT(TSharedPtr<DragDropWidgetType>, Widget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, HeldItemType InItem)
	{
		WidgetWeakPtr = InArgs._Widget;
		Item = MoveTemp(InItem);
		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(25.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
			]
		];
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			if (const TSharedPtr<FDragDropOperation> DragDropOp = CreateDragDropOperation())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}
		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> CreateDragDropOperation()
	{
		if (const TSharedPtr<DragDropWidgetType> Widget = WidgetWeakPtr.Pin())
		{
			TSharedPtr<DragDropType> DragDropOperation = MakeShared<DragDropType>(Widget, Item);
			DragDropOperation->Construct();
			return DragDropOperation;
		}
		return nullptr;
	}

private:
	/** Holds the widget to display when dragging. */
	TWeakPtr<DragDropWidgetType> WidgetWeakPtr;

	/** Holds the item being dragged. */
	HeldItemType Item;
};
