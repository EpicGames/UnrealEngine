// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Components/SRenderGridEditableTextBlock.h"

#include "EditorFontGlyphs.h"
#include "PropertyCustomizationHelpers.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRenderGridEditableTextBlock"


void UE::RenderGrid::Private::SRenderGridEditableTextBlock::Tick(const FGeometry&, const double, const float)
{
	if (bNeedsRename)
	{
		if (TextBlock)
		{
			TextBlock->EnterEditingMode();
		}
		bNeedsRename = false;
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridEditableTextBlock::Construct(const FArguments& InArgs)
{
	bNeedsRename = false;
	OnTextCommittedDelegate = InArgs._OnTextCommitted;

	SetText(InArgs._Text);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		// Text block
		.FillWidth(1.f)
		.VAlign(VAlign_Fill)
		.Padding(2.f)
		[
			SAssignNew(TextBlock, SInlineEditableTextBlock)
			.Text(Text)
			.OnTextCommitted(this, &SRenderGridEditableTextBlock::OnTextBlockCommitted)
		]
		+ SHorizontalBox::Slot()
		// Button
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.OnClicked_Lambda([this]()
			{
				bNeedsRename = true;
				return FReply::Handled();
			})
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FEditorFontGlyphs::Pencil /*fa-pencil*/)
			]
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridEditableTextBlock::SetText(TAttribute<FText> InText)
{
	SetText(InText.IsSet() ? InText.Get() : FText());
}

void UE::RenderGrid::Private::SRenderGridEditableTextBlock::SetText(const FText& InText)
{
	Text = InText;

	if (TextBlock.IsValid())
	{
		TextBlock->SetText(Text);
	}
}

void UE::RenderGrid::Private::SRenderGridEditableTextBlock::OnTextBlockCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	SetText(InLabel);
	if (OnTextCommittedDelegate.IsBound())
	{
		SetText(OnTextCommittedDelegate.Execute(InLabel, InCommitInfo));
	}
}


#undef LOCTEXT_NAMESPACE
