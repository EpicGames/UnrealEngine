// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


namespace UE::RenderGrid::Private
{
	DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnRenderGridEditableTextBlockTextCommitted, const FText&, ETextCommit::Type);

	/**
	 * A reusable text widget that can be modified by pressing a pencil button that's shown next to it.
	 */
	class SRenderGridEditableTextBlock : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridEditableTextBlock) {}
			SLATE_ATTRIBUTE(FText, Text)// The text displayed in this text block
			SLATE_EVENT(FOnRenderGridEditableTextBlockTextCommitted, OnTextCommitted)// Called when the text is committed
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs);

		/**
		 * Gets the text of this text block
		 *
		 * @return	This text block's text string
		 */
		const FText& GetText() const { return Text; }

	public:
		/** Sets the text for this text block */
		void SetText(TAttribute<FText> InText);

		/** Sets the text for this text block */
		void SetText(const FText& InText);

	protected:
		void OnTextBlockCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);

	protected:
		/** A reference to the text block. */
		TSharedPtr<SInlineEditableTextBlock> TextBlock;

		/** The current text in the text block (read-only). */
		FText Text;

		/** Whether the group needs to be renamed. (As requested by a click on the rename button) */
		bool bNeedsRename;

		/** Delegate to execute when editing mode text is committed. */
		FOnRenderGridEditableTextBlockTextCommitted OnTextCommittedDelegate;
	};
}
