// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Components/SRenderGridFileSelectorTextBlock.h"

#include "DesktopPlatformModule.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyCustomizationHelpers.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRenderGridFileSelectorTextBlock"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridFileSelectorTextBlock::Construct(const FArguments& InArgs)
{
	FolderPath = InArgs._FolderPath;
	OnTextCommittedDelegate = InArgs._OnTextCommitted;

	SetText(InArgs._Text);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		// Text block
		.FillWidth(1.f)
		.VAlign(VAlign_Fill)
		.Padding(2.f)
		[
			SAssignNew(TextBlock, SInlineEditableTextBlock)
			.IsEnabled(false)
			.Text(Text)
		]
		+ SHorizontalBox::Slot()
		// Button
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.OnClicked(this, &SRenderGridFileSelectorTextBlock::OnOpenDirectoryDialog)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FEditorFontGlyphs::Folder_Open /*fa-folder-open*/)
			]
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridFileSelectorTextBlock::SetText(TAttribute<FText> InText)
{
	SetText(InText.IsSet() ? InText.Get() : FText());
}

void UE::RenderGrid::Private::SRenderGridFileSelectorTextBlock::SetText(const FText& InText)
{
	Text = InText;

	if (TextBlock.IsValid())
	{
		TextBlock->SetText(Text);
	}
}

FReply UE::RenderGrid::Private::SRenderGridFileSelectorTextBlock::OnOpenDirectoryDialog()
{
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		if (FSlateApplication::IsInitialized())
		{
			TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

			FString OutFolderPath;
			const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
				ParentWindowHandle,
				LOCTEXT("Title_OpenDirectoryDialog", "Choose a directory").ToString(),
				FolderPath.Get(FPaths::ProjectDir() / TEXT("Saved/MovieRenders/")),
				OutFolderPath
			);

			if (bFolderSelected)
			{
				const FText NewText = FText::FromString(OutFolderPath);
				SetText(NewText);
				if (OnTextCommittedDelegate.IsBound())
				{
					SetText(OnTextCommittedDelegate.Execute(NewText, ETextCommit::Type::OnEnter));
				}
			}
		}
	}
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
