// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


namespace UE::RenderGrid::Private
{
	DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnRenderGridFileSelectorTextBlockTextCommitted, const FText&, ETextCommit::Type);

	/**
	 * A reusable filepath text widget that can be modified by pressing a folder button that's shown next to it, which will open up a file selector popup.
	 */
	class SRenderGridFileSelectorTextBlock : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridFileSelectorTextBlock) {}
			SLATE_ATTRIBUTE(FText, Text)// The text displayed in this text block
			SLATE_ATTRIBUTE(FString, FolderPath)// The default path of the open-directory dialog
			SLATE_EVENT(FOnRenderGridFileSelectorTextBlockTextCommitted, OnTextCommitted)// Callback when the text is committed
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/**
		 * Gets the text of this text block
		 *
		 * @return	This text block's text string
		 */
		const FText& GetText() const { return Text; }

	public:
		/** Sets the text for this text block */
		void SetText(TAttribute<FText> InText);

		/** Sets the text for this text block */
		void SetText(const FText& InText);

	protected:
		FReply OnOpenDirectoryDialog();

	protected:
		/** A reference to the text block. */
		TSharedPtr<SInlineEditableTextBlock> TextBlock;

		/** The current text in the text block (read-only). */
		FText Text;

		/** The default path of the open-directory dialog (read-only). */
		TAttribute<FString> FolderPath;

		/** Delegate to execute when editing mode text is committed. */
		FOnRenderGridFileSelectorTextBlockTextCommitted OnTextCommittedDelegate;
	};
}
