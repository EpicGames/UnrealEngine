// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenderGridRemoteControlEntity.h"
#include "RemoteControlBinding.h"
#include "RemoteControlEntity.h"
#include "RemoteControlPreset.h"
#include "RemoteControlField.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SRenderGridExposedEntity"


TSharedPtr<FRemoteControlEntity> UE::RenderGrid::Private::SRenderGridRemoteControlEntity::GetEntity() const
{
	if (const TSharedPtr<FRemoteControlEntity> Entity = PresetWeakPtr->GetExposedEntity(EntityId).Pin())
	{
		return Entity;
	}
	return nullptr;
}

void UE::RenderGrid::Private::SRenderGridRemoteControlEntity::Initialize(const FGuid& InEntityId, URemoteControlPreset* InPreset)
{
	EntityId = InEntityId;
	PresetWeakPtr = InPreset;

	if (ensure(InPreset))
	{
		if (const TSharedPtr<FRemoteControlEntity> RCEntity = InPreset->GetExposedEntity(InEntityId).Pin())
		{
			CachedLabel = RCEntity->GetLabel();
		}
	}
}

TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridRemoteControlEntity::CreateEntityWidget(const TSharedPtr<SWidget> ValueWidget)
{
	FRenderGridMakeNodeWidgetArgs Args;

	TSharedRef<SBorder> Widget = SNew(SBorder)
		.Padding(0.0f)
		.BorderImage(new FSlateNoResource());

	Args.NameWidget = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SInlineEditableTextBlock)
			.Text(FText::FromName(CachedLabel))
			.IsReadOnly(true)
		];

	Args.ValueWidget = ValueWidget;

	Widget->SetContent(MakeNodeWidget(Args));
	return Widget;
}


#undef LOCTEXT_NAMESPACE
