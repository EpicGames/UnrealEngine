// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SRenderGridRemoteControlTreeNode.h"


class SWidget;
struct FRemoteControlEntity;
class URemoteControlPreset;


namespace UE::RenderGrid::Private
{
	/**
	 * The remote control entity widget, copied over from the remote control plugin, and slightly modified and cleaned up for usage in the render grid plugin.
	 *
	 * @Note if you inherit from this struct, you must call SRenderGridExposedEntity::Initialize.
	 */
	struct SRenderGridRemoteControlEntity : SRenderGridRemoteControlTreeNode
	{
		//~ SRenderGridTreeNode interface
		TSharedPtr<FRemoteControlEntity> GetEntity() const;
		virtual FGuid GetRCId() const override final { return EntityId; }
		//~ End SRenderGridTreeNode interface

	protected:
		void Initialize(const FGuid& InEntityId, URemoteControlPreset* InPreset);

		/** Create an exposed entity widget. */
		TSharedRef<SWidget> CreateEntityWidget(const TSharedPtr<SWidget> ValueWidget);

	protected:
		/** Id of the entity. */
		FGuid EntityId;

		/** The underlying preset. */
		TWeakObjectPtr<URemoteControlPreset> PresetWeakPtr;

		/** Display name of the entity. */
		FName CachedLabel;
	};
}
