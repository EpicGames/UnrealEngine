// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenderGridRemoteControlField.h"
#include "UI/Utils/RenderGridWidgetUtils.h"
#include "Algo/Transform.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "SRenderGridRemoteControlTreeNode.h"
#include "UObject/Object.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SRenderGridExposedField"


namespace ExposedFieldUtils
{
	TSharedRef<SWidget> CreateNodeValueWidget(const FNodeWidgets& NodeWidgets)
	{
		TSharedRef<SHorizontalBox> FieldWidget = SNew(SHorizontalBox);

		if (NodeWidgets.ValueWidget)
		{
			FieldWidget->AddSlot()
				.Padding(FMargin(3.0f, 2.0f))
				.HAlign(HAlign_Right)
				.FillWidth(1.0f)
				[
					NodeWidgets.ValueWidget.ToSharedRef()
				];
		}
		else if (NodeWidgets.WholeRowWidget)
		{
			FieldWidget->AddSlot()
				.Padding(FMargin(3.0f, 2.0f))
				.FillWidth(1.0f)
				[
					NodeWidgets.WholeRowWidget.ToSharedRef()
				];
		}

		return FieldWidget;
	}
}

TSharedPtr<UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode> UE::RenderGrid::Private::SRenderGridRemoteControlField::MakeInstance(const FRenderGridRemoteControlGenerateWidgetArgs& Args)
{
	return SNew(SRenderGridRemoteControlField, StaticCastSharedPtr<FRemoteControlField>(Args.Entity), Args.ColumnSizeData).Preset(Args.Preset);
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::Tick(const FGeometry&, const double, const float)
{
	if (FramesUntilRerender > 0)
	{
		FramesUntilRerender--;
		if (FramesUntilRerender <= 0)
		{
			Refresh();
		}
	}
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlField> InField, FRenderGridRemoteControlColumnSizeData InColumnSizeData)
{
	FieldWeakPtr = MoveTemp(InField);
	ColumnSizeData = MoveTemp(InColumnSizeData);
	FramesUntilRerender = 0;

	if (const TSharedPtr<FRemoteControlField> Field = FieldWeakPtr.Pin())
	{
		Initialize(Field->GetId(), InArgs._Preset.Get());

		CachedLabel = Field->GetLabel();
		EntityId = Field->GetId();

		if (Field->FieldType == EExposedFieldType::Property)
		{
			ConstructPropertyWidget();

			if (Generator.IsValid())
			{
				Generator.Get()->OnRowsRefreshed().RemoveAll(this);
				Generator.Get()->OnRowsRefreshed().AddLambda([this]()
				{
					Generator.Get()->OnRowsRefreshed().RemoveAll(this);
					FramesUntilRerender = 2;
				});
			}
		}
	}
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::GetNodeChildren(TArray<TSharedPtr<SRenderGridRemoteControlTreeNode>>& OutChildren) const
{
	OutChildren.Append(ChildWidgets);
}

UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::ENodeType UE::RenderGrid::Private::SRenderGridRemoteControlField::GetRCType() const
{
	return ENodeType::Field;
}

FName UE::RenderGrid::Private::SRenderGridRemoteControlField::GetFieldLabel() const
{
	return CachedLabel;
}

EExposedFieldType UE::RenderGrid::Private::SRenderGridRemoteControlField::GetFieldType() const
{
	if (const TSharedPtr<FRemoteControlField> Field = FieldWeakPtr.Pin())
	{
		return Field->FieldType;
	}
	return EExposedFieldType::Invalid;
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::Refresh()
{
	if (const TSharedPtr<FRemoteControlField> Field = FieldWeakPtr.Pin())
	{
		CachedLabel = Field->GetLabel();

		if (Field->FieldType == EExposedFieldType::Property)
		{
			ConstructPropertyWidget();
		}
	}
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::RefreshValue()
{
	if (!Generator.IsValid() || (GetFieldType() != EExposedFieldType::Property))
	{
		Refresh();
	}
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::GetBoundObjects(TSet<UObject*>& OutBoundObjects) const
{
	if (const TSharedPtr<FRemoteControlField> Field = FieldWeakPtr.Pin())
	{
		OutBoundObjects.Append(Field->GetBoundObjects());
	}
}

TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridRemoteControlField::ConstructWidget()
{
	if (const TSharedPtr<FRemoteControlField> Field = FieldWeakPtr.Pin())
	{
		// For the moment, just use the first object.
		TArray<UObject*> Objects = Field->GetBoundObjects();
		if ((GetFieldType() == EExposedFieldType::Property) && (Objects.Num() > 0))
		{
			FPropertyRowGeneratorArgs Args;
			Generator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
			Generator->SetObjects({Objects[0]});

			if (const TSharedPtr<IDetailTreeNode> Node = RenderGridWidgetUtils::FindNode(Generator->GetRootTreeNodes(), Field->FieldPathInfo.ToPathPropertyString(), RenderGridWidgetUtils::ERenderGridFindNodeMethod::Path))
			{
				TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
				Node->GetChildren(ChildNodes);
				ChildWidgets.Reset(ChildNodes.Num());

				for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
				{
					ChildWidgets.Add(SNew(SRenderGridRemoteControlFieldChildNode, ChildNode, ColumnSizeData));
				}

				return MakeFieldWidget(ExposedFieldUtils::CreateNodeValueWidget(Node->CreateNodeWidgets()));
			}
		}
	}

	return MakeFieldWidget(SNullWidget::NullWidget);
}

TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridRemoteControlField::MakeFieldWidget(const TSharedRef<SWidget>& InWidget)
{
	return CreateEntityWidget(InWidget);
}

void UE::RenderGrid::Private::SRenderGridRemoteControlField::ConstructPropertyWidget()
{
	ChildSlot.AttachWidget(ConstructWidget());
}


void UE::RenderGrid::Private::SRenderGridRemoteControlFieldChildNode::Construct(const FArguments& InArgs, const TSharedRef<IDetailTreeNode>& InNode, FRenderGridRemoteControlColumnSizeData InColumnSizeData)
{
	TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
	InNode->GetChildren(ChildNodes);

	Algo::Transform(ChildNodes, ChildrenNodes, [InColumnSizeData](const TSharedRef<IDetailTreeNode>& ChildNode) { return SNew(SRenderGridRemoteControlFieldChildNode, ChildNode, InColumnSizeData); });

	ColumnSizeData = InColumnSizeData;

	FNodeWidgets Widgets = InNode->CreateNodeWidgets();
	FRenderGridMakeNodeWidgetArgs Args;
	Args.NameWidget = Widgets.NameWidget;
	Args.ValueWidget = ExposedFieldUtils::CreateNodeValueWidget(Widgets);

	ChildSlot
	[
		MakeNodeWidget(Args)
	];
}


#undef LOCTEXT_NAMESPACE
