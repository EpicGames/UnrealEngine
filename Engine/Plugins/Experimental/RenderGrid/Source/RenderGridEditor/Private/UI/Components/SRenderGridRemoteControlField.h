// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RemoteControlField.h"
#include "UI/Components/SRenderGridRemoteControlEntity.h"
#include "UI/Components/SRenderGridRemoteControlTreeNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class IDetailTreeNode;
class IPropertyRowGenerator;
class URemoteControlPreset;
enum class EExposedFieldType : uint8;
struct FGuid;
struct FRemoteControlField;
struct FRenderGridGenerateWidgetArgs;


namespace UE::RenderGrid::Private
{
	/**
	 * The remote control field child node widget, copied over from the remote control plugin, and slightly modified and cleaned up for usage in the render grid plugin.
	 *
	 * Represents a child of an exposed field widget.
	 */
	struct SRenderGridRemoteControlFieldChildNode : SRenderGridRemoteControlTreeNode
	{
		SLATE_BEGIN_ARGS(SRenderGridRemoteControlFieldChildNode) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<IDetailTreeNode>& InNode, FRenderGridRemoteControlColumnSizeData InColumnSizeData);
		virtual void GetNodeChildren(TArray<TSharedPtr<SRenderGridRemoteControlTreeNode>>& OutChildren) const override { return OutChildren.Append(ChildrenNodes); }
		virtual FGuid GetRCId() const override { return FGuid(); }
		virtual ENodeType GetRCType() const override { return ENodeType::FieldChild; }

		TArray<TSharedPtr<SRenderGridRemoteControlFieldChildNode>> ChildrenNodes;
	};


	/**
	 * The remote control field widget, copied over from the remote control plugin, and slightly modified and cleaned up for usage in the render grid plugin.
	 *
	 * Widget that displays an exposed field.
	 */
	struct SRenderGridRemoteControlField : SRenderGridRemoteControlEntity
	{
		SLATE_BEGIN_ARGS(SRenderGridRemoteControlField)
				: _Preset(nullptr) {}
			SLATE_ATTRIBUTE(URemoteControlPreset*, Preset)
		SLATE_END_ARGS()

		static TSharedPtr<SRenderGridRemoteControlTreeNode> MakeInstance(const FRenderGridRemoteControlGenerateWidgetArgs& Args);

		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		void Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlField> Field, FRenderGridRemoteControlColumnSizeData ColumnSizeData);

		//~ SRenderGridTreeNode Interface 
		virtual void GetNodeChildren(TArray<TSharedPtr<SRenderGridRemoteControlTreeNode>>& OutChildren) const override;
		virtual ENodeType GetRCType() const override;
		virtual void Refresh() override;
		virtual void RefreshValue() override;
		//~ End SRenderGridTreeNode Interface

		/** Get a weak pointer to the underlying remote control field. */
		TWeakPtr<FRemoteControlField> GetRemoteControlField() const { return FieldWeakPtr; }

		/** Get this field's label. */
		FName GetFieldLabel() const;

		/** Get this field's type. */
		EExposedFieldType GetFieldType() const;

		/** Returns this widget's underlying objects. */
		void GetBoundObjects(TSet<UObject*>& OutBoundObjects) const;

	private:
		/** Construct a property widget. */
		TSharedRef<SWidget> ConstructWidget();

		/** Create the wrapper around the field value widget. */
		TSharedRef<SWidget> MakeFieldWidget(const TSharedRef<SWidget>& InWidget);

		/** Construct this field widget as a property widget. */
		void ConstructPropertyWidget();

	private:
		/** Weak pointer to the underlying RC Field. */
		TWeakPtr<FRemoteControlField> FieldWeakPtr;

		/** This exposed field's child widgets (ie. An array's rows) */
		TArray<TSharedPtr<SRenderGridRemoteControlFieldChildNode>> ChildWidgets;

		/** The property row generator. */
		TSharedPtr<IPropertyRowGenerator> Generator;

		/** Has a value of 1 if it should call Render() next frame, 2+ if it should subtract 1 from its value next frame, 0 and below and it won't do anything. */
		int32 FramesUntilRerender;
	};
}
