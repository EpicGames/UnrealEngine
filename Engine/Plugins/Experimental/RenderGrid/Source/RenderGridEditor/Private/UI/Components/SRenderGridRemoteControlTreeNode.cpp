// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenderGridRemoteControlTreeNode.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SRenderGridTreeNode"


TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::MakeSplitRow(TSharedRef<SWidget> LeftColumn, TSharedRef<SWidget> RightColumn)
{
	TAttribute<float> LeftColumnAttribute;
	LeftColumnAttribute.BindRaw(this, &SRenderGridRemoteControlTreeNode::GetLeftColumnWidth);
	TAttribute<float> RightColumnAttribute;
	RightColumnAttribute.BindRaw(this, &SRenderGridRemoteControlTreeNode::GetRightColumnWidth);

	return SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(1.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		+ SSplitter::Slot()
		.Value(MoveTemp(LeftColumnAttribute))
		.OnSlotResized(SSplitter::FOnSlotResized::CreateRaw(this, &SRenderGridRemoteControlTreeNode::OnLeftColumnResized))
		[
			LeftColumn
		]
		+ SSplitter::Slot()
		.Value(MoveTemp(RightColumnAttribute))
		.OnSlotResized(SSplitter::FOnSlotResized::CreateRaw(this, &SRenderGridRemoteControlTreeNode::SetColumnWidth))
		[
			RightColumn
		];
}

TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::MakeNodeWidget(const FRenderGridMakeNodeWidgetArgs& Args)
{
	auto WidgetOrNull = [](const TSharedPtr<SWidget>& Widget) { return Widget ? Widget.ToSharedRef() : SNullWidget::NullWidget; };

	TSharedRef<SWidget> LeftColumn =
		SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		// Name
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			WidgetOrNull(Args.NameWidget)
		];

	TSharedRef<SWidget> RightColumn =
		SNew(SOverlay)
		// Value
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		[
			SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				WidgetOrNull(Args.ValueWidget)
			]
		];

	return MakeSplitRow(LeftColumn, RightColumn);
}

void UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::OnLeftColumnResized(float) const
{
	// This has to be bound or the splitter will take it upon itself to determine the size
	// We do nothing here because it is handled by the column size data
}

float UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::GetLeftColumnWidth() const
{
	const float Offset = (GetRCType() == ENodeType::Group) ? SplitterOffset : 0;
	return FMath::Clamp(ColumnSizeData.LeftColumnWidth.Get() + Offset, 0.f, 1.f);
}

float UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::GetRightColumnWidth() const
{
	const float Offset = (GetRCType() == ENodeType::Group) ? SplitterOffset : 0;
	return FMath::Clamp(ColumnSizeData.RightColumnWidth.Get() - Offset, 0.f, 1.f);
}

void UE::RenderGrid::Private::SRenderGridRemoteControlTreeNode::SetColumnWidth(float InWidth)
{
	ColumnSizeData.SetColumnWidth(FMath::Clamp(InWidth + SplitterOffset, 0.f, 1.f));
}


#undef LOCTEXT_NAMESPACE
