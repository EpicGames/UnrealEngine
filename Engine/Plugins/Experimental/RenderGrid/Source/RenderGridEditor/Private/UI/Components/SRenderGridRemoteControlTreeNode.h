// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Misc/Guid.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SCompoundWidget.h"


class SWidget;
class URemoteControlPreset;
struct FRemoteControlEntity;


namespace UE::RenderGrid::Private
{
	/**
	 * The remote control column size data struct, copied over from the remote control plugin, and slightly modified and cleaned up for usage in the render grid plugin.
	 */
	struct FRenderGridRemoteControlColumnSizeData
	{
		TAttribute<float> LeftColumnWidth = 0.0f;
		TAttribute<float> RightColumnWidth = 0.0f;
		SSplitter::FOnSlotResized OnWidthChanged;

		void SetColumnWidth(const float InWidth)
		{
			OnWidthChanged.ExecuteIfBound(InWidth);
		}

		bool operator!=(const FRenderGridRemoteControlColumnSizeData& RHS) const { return !(*this == RHS); }
		bool operator==(const FRenderGridRemoteControlColumnSizeData& RHS) const
		{
			return LeftColumnWidth.IdenticalTo(RHS.LeftColumnWidth) && RightColumnWidth.IdenticalTo(RHS.RightColumnWidth);
		}
	};

	/**
	 * The remote control generate widget args struct, copied over from the remote control plugin, and slightly modified and cleaned up for usage in the render grid plugin.
	 */
	struct FRenderGridRemoteControlGenerateWidgetArgs
	{
		TObjectPtr<URemoteControlPreset> Preset = nullptr;
		FRenderGridRemoteControlColumnSizeData ColumnSizeData;
		TSharedPtr<FRemoteControlEntity> Entity = nullptr;
		FName EntityFieldLabel;

		bool operator!=(const FRenderGridRemoteControlGenerateWidgetArgs& RHS) const { return !(*this == RHS); }
		bool operator==(const FRenderGridRemoteControlGenerateWidgetArgs& RHS) const
		{
			return (Preset == RHS.Preset) && (ColumnSizeData == RHS.ColumnSizeData) && (Entity == RHS.Entity) && (EntityFieldLabel == RHS.EntityFieldLabel);
		}
	};


	/**
	 * The remote control tree node widget, copied over from the remote control plugin, and slightly modified and cleaned up for usage in the render grid plugin.
	 *
	 * A node in the panel tree view.
	 */
	struct SRenderGridRemoteControlTreeNode : SCompoundWidget
	{
		enum class ENodeType : uint8
		{
			Invalid,
			Group,
			Field,
			FieldChild,
			Actor,
			Material
		};

		/** Get this tree node's children. */
		virtual void GetNodeChildren(TArray<TSharedPtr<SRenderGridRemoteControlTreeNode>>& OutChildren) const {}

		/** Get this node's ID if any. */
		virtual FGuid GetRCId() const { return FGuid(); }

		/** Get get this node's type. */
		virtual ENodeType GetRCType() const { return ENodeType::Invalid; }

		/** Refresh the node. */
		virtual void Refresh() {}

		/** Refreshes the value of the node, without replacing the node. */
		virtual void RefreshValue() {}

	protected:
		struct FRenderGridMakeNodeWidgetArgs
		{
			TSharedPtr<SWidget> DragHandle;
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> RenameButton;
			TSharedPtr<SWidget> ValueWidget;
			TSharedPtr<SWidget> UnexposeButton;
		};

		/** Create a widget that represents a row with a splitter. */
		TSharedRef<SWidget> MakeSplitRow(TSharedRef<SWidget> LeftColumn, TSharedRef<SWidget> RightColumn);

		/** Create a widget that represents a node in the panel tree hierarchy. */
		TSharedRef<SWidget> MakeNodeWidget(const FRenderGridMakeNodeWidgetArgs& Args);

	private:
		/** Stub handler for column resize callback to prevent the splitter from handling it internally.  */
		void OnLeftColumnResized(float) const;

		//~ Wrappers around ColumnSizeData's delegate needed in order to offset the splitter for RC Groups. 
		float GetLeftColumnWidth() const;
		float GetRightColumnWidth() const;
		void SetColumnWidth(float InWidth);

	protected:
		/** Holds the row's columns' width. */
		FRenderGridRemoteControlColumnSizeData ColumnSizeData;

	private:
		/** The splitter offset to align the group splitter with the other row's splitters. */
		static constexpr float SplitterOffset = 0.008f;
	};
}
