// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Components/SRenderGridViewerFrameSlider.h"
#include "RenderGrid/RenderGrid.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRenderGridViewerFrameSlider"


float UE::RenderGrid::Private::SRenderGridViewerFrameSlider::FrameSliderValue = 0;


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewerFrameSlider::Construct(const FArguments& InArgs)
{
	OnValueChanged = InArgs._OnValueChanged;
	OnCaptureEnd = InArgs._OnCaptureEnd;

	SAssignNew(FrameSlider, SSlider)
		.IndentHandle(true)
		.MouseUsesStep(false)
		.StepSize(0.0001)
		.MinValue(0)
		.MaxValue(1)
		.Value(FrameSliderValue)
		.OnValueChanged(this, &SRenderGridViewerFrameSlider::FrameSliderValueChanged)
		.OnMouseCaptureEnd(this, &SRenderGridViewerFrameSlider::FrameSliderValueChangedEnd)
		.OnControllerCaptureEnd(this, &SRenderGridViewerFrameSlider::FrameSliderValueChangedEnd);

	SAssignNew(FrameSliderStartFrameText, STextBlock);
	SAssignNew(FrameSliderSelectedFrameText, STextBlock);
	SAssignNew(FrameSliderEndFrameText, STextBlock);

	ChildSlot
	[
		SNew(SVerticalBox)

		// slider
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.HeightOverride(39.0f)
			[
				FrameSlider.ToSharedRef()
			]
		]

		// slider text
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f, -14.0f, 5.0f, 3.0f)
		[
			SNew(SHorizontalBox)

			// start frame
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			[
				FrameSliderStartFrameText.ToSharedRef()
			]

			// selected frame
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			[
				FrameSliderSelectedFrameText.ToSharedRef()
			]

			// end frame
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			[
				FrameSliderEndFrameText.ToSharedRef()
			]
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewerFrameSlider::FrameSliderValueChanged(const float NewValue)
{
	FrameSliderValue = NewValue;
	OnValueChanged.ExecuteIfBound(FrameSliderValue);
}

void UE::RenderGrid::Private::SRenderGridViewerFrameSlider::FrameSliderValueChangedEnd()
{
	OnCaptureEnd.ExecuteIfBound();
}


void UE::RenderGrid::Private::SRenderGridViewerFrameSlider::ClearFramesText()
{
	if (FrameSliderStartFrameText.IsValid())
	{
		FrameSliderStartFrameText->SetText(FText());
	}
	if (FrameSliderSelectedFrameText.IsValid())
	{
		FrameSliderSelectedFrameText->SetText(FText());
	}
	if (FrameSliderEndFrameText.IsValid())
	{
		FrameSliderEndFrameText->SetText(FText());
	}
}

void UE::RenderGrid::Private::SRenderGridViewerFrameSlider::SetFramesText(const int32 StartFrame, const int32 SelectedFrame, const int32 EndFrame)
{
	static const FNumberFormattingOptions FrameNumberFormat = FNumberFormattingOptions()
		.SetUseGrouping(false)
		.SetMinimumIntegralDigits(4);

	if (FrameSliderStartFrameText.IsValid())
	{
		FrameSliderStartFrameText->SetText(FText::AsNumber(StartFrame, &FrameNumberFormat));
	}
	if (FrameSliderSelectedFrameText.IsValid())
	{
		FrameSliderSelectedFrameText->SetText(FText::AsNumber(SelectedFrame, &FrameNumberFormat));
	}
	if (FrameSliderEndFrameText.IsValid())
	{
		FrameSliderEndFrameText->SetText(FText::AsNumber(EndFrame, &FrameNumberFormat));
	}
}

bool UE::RenderGrid::Private::SRenderGridViewerFrameSlider::SetFramesText(URenderGridJob* Job)
{
	if (!IsValid(Job) || !FrameSlider.IsValid())
	{
		return false;
	}

	TOptional<int32> SelectedFrame = GetSelectedFrame(Job);
	TOptional<int32> StartFrame = Job->GetStartFrame();
	TOptional<int32> EndFrame = Job->GetEndFrame();
	if (!SelectedFrame.IsSet() || !StartFrame.IsSet() || !EndFrame.IsSet() || (StartFrame.Get(0) >= EndFrame.Get(0)))
	{
		return false;
	}

	SetFramesText(
		StartFrame.Get(0),
		SelectedFrame.Get(0),
		EndFrame.Get(0) - 1
	);
	return true;
}

TOptional<int32> UE::RenderGrid::Private::SRenderGridViewerFrameSlider::GetSelectedSequenceFrame(URenderGridJob* Job)
{
	if (!IsValid(Job) || !FrameSlider.IsValid())
	{
		return TOptional<int32>();
	}

	TOptional<int32> OptionalStartFrame = Job->GetSequenceStartFrame();
	TOptional<int32> OptionalEndFrame = Job->GetSequenceEndFrame();
	int32 StartFrame = OptionalStartFrame.Get(0);
	int32 EndFrame = OptionalEndFrame.Get(0) - 1;
	if (!OptionalStartFrame.IsSet() || !OptionalEndFrame.IsSet() || (StartFrame >= EndFrame))
	{
		return TOptional<int32>();
	}

	return FMath::RoundToInt(FMath::Lerp(static_cast<double>(StartFrame), static_cast<double>(EndFrame), FrameSlider->GetValue()));
}

TOptional<int32> UE::RenderGrid::Private::SRenderGridViewerFrameSlider::GetSelectedFrame(URenderGridJob* Job)
{
	if (!IsValid(Job) || !FrameSlider.IsValid())
	{
		return TOptional<int32>();
	}

	TOptional<int32> OptionalStartFrame = Job->GetStartFrame();
	TOptional<int32> OptionalEndFrame = Job->GetEndFrame();
	int32 StartFrame = OptionalStartFrame.Get(0);
	int32 EndFrame = OptionalEndFrame.Get(0) - 1;
	if (!OptionalStartFrame.IsSet() || !OptionalEndFrame.IsSet() || (StartFrame >= EndFrame))
	{
		return TOptional<int32>();
	}

	return FMath::RoundToInt(FMath::Lerp(static_cast<double>(StartFrame), static_cast<double>(EndFrame), FrameSlider->GetValue()));
}


#undef LOCTEXT_NAMESPACE
