// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"


class SBox;
class SSlider;
class STextBlock;
class URenderGridJob;


namespace UE::RenderGrid::Private
{
	/**
	 * The frame slider widget for the render grid viewer widgets.
	 */
	class SRenderGridViewerFrameSlider : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridViewerFrameSlider) {}
			SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)// Called when the value is changed by the slider
			SLATE_EVENT(FSimpleDelegate, OnCaptureEnd)// Called when the capture ends
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		/** Returns the current value of the frame slider (a value from 0.0 to 1.0, based on how far the slider is to the right, left=0.0, right=1.0). */
		float GetValue() { return FrameSliderValue; }

		/** Clears the text of the start, current and end frame. */
		void ClearFramesText();

		/** Sets the text of the start, current and end frame. */
		void SetFramesText(const int32 StartFrame, const int32 SelectedFrame, const int32 EndFrame);

		/** Sets the text of the start, current and end frame according to the start and end frame of the given render grid job, figures out the selected frame with the current value of the frame slider. */
		bool SetFramesText(URenderGridJob* Job);

		/** Gets the selected sequence frame of the given render grid job, based on the current value of the frame slider. */
		TOptional<int32> GetSelectedSequenceFrame(URenderGridJob* Job);

		/** Gets the selected frame (that will be output) of the given render grid job, based on the current value of the frame slider. */
		TOptional<int32> GetSelectedFrame(URenderGridJob* Job);

	private:
		void FrameSliderValueChanged(const float NewValue);
		void FrameSliderValueChangedEnd();

	private:
		/** The widget that allows the user to select what frame they'd like to see. */
		TSharedPtr<SSlider> FrameSlider;

	private:
		/** The value of the frame slider. */
		static float FrameSliderValue;

		/** The text for the start frame under the slider. */
		TSharedPtr<STextBlock> FrameSliderStartFrameText;

		/** The text for the selected frame under the slider. */
		TSharedPtr<STextBlock> FrameSliderSelectedFrameText;

		/** The text for the end frame under the slider. */
		TSharedPtr<STextBlock> FrameSliderEndFrameText;

		/** Called when the value is changed by the slider. */
		FOnFloatValueChanged OnValueChanged;

		/** Called when the capture ends. */
		FSimpleDelegate OnCaptureEnd;
	};
}
