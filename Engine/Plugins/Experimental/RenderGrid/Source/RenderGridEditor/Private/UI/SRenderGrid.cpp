// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGrid.h"
#include "IRenderGridEditor.h"
#include "RenderGrid/RenderGrid.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SlateOptMacros.h"
#include "Blueprints/RenderGridBlueprint.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SRenderGrid"


void UE::RenderGrid::Private::SRenderGrid::Tick(const FGeometry&, const double, const float)
{
	if (RenderGridDetailsView.IsValid())
	{
		if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
		{
			if (URenderGrid* Grid = BlueprintEditor->GetInstance())
			{
				if (!IsValid(Grid) || BlueprintEditor->ShouldHideUI())
				{
					Grid = nullptr;
				}
				if (RenderGridWeakPtr != Grid)
				{
					SetRenderGrid(Grid);
				}
			}
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGrid::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	RenderGridWeakPtr = nullptr;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.NotifyHook = this;

	RenderGridDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	RenderGridSettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	RenderGridDefaultsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	SetRenderGrid(InBlueprintEditor->GetInstance());

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			RenderGridDetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			RenderGridSettingsDetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			RenderGridDefaultsDetailsView->AsShared()
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGrid::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) {}

void UE::RenderGrid::Private::SRenderGrid::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		FScopedTransaction Transaction(LOCTEXT("RenderGridParameterChanged", "RenderGrid Parameter Changed"));
		BlueprintEditor->MarkAsModified();
		BlueprintEditor->GetRenderGridBlueprint()->PropagateAllPropertiesExceptJobsToAsset(BlueprintEditor->GetInstance());
		BlueprintEditor->OnRenderGridChanged().Broadcast();
	}
}

void UE::RenderGrid::Private::SRenderGrid::SetRenderGrid(URenderGrid* RenderGrid)
{
	RenderGridWeakPtr = RenderGrid;
	RenderGridDetailsView->SetObject(RenderGrid);
	RenderGridSettingsDetailsView->SetObject(IsValid(RenderGrid) ? RenderGrid->GetSettingsObject() : nullptr);
	RenderGridDefaultsDetailsView->SetObject(IsValid(RenderGrid) ? RenderGrid->GetDefaultsObject() : nullptr);
}


#undef LOCTEXT_NAMESPACE
