// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"


class URenderGrid;
class URenderGridDefaults;
class IDetailsView;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * A widget with which the user can modify the render grid.
	 * Doesn't contain any UI elements to modify the jobs that the grid contains.
	 */
	class SRenderGrid : public SCompoundWidget, public FNotifyHook
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGrid) {}
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

		//~ Begin FNotifyHook Interface
		virtual void NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) override;
		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;
		//~ End FNotifyHook Interface

	protected:
		void SetRenderGrid(URenderGrid* RenderGrid);

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The render grid instance that's being edited in the details view. */
		TWeakObjectPtr<URenderGrid> RenderGridWeakPtr;

		/** A reference to the details view. */
		TSharedPtr<IDetailsView> RenderGridDetailsView;

		/** A reference to the details view. */
		TSharedPtr<IDetailsView> RenderGridSettingsDetailsView;

		/** A reference to the details view. */
		TSharedPtr<IDetailsView> RenderGridDefaultsDetailsView;
	};
}
