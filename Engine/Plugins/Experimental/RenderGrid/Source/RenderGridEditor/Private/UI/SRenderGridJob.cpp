// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridJob.h"
#include "RenderGrid/RenderGrid.h"
#include "IRenderGridEditor.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SRenderGridJob"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridJob::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(nullptr);

	Refresh();
	InBlueprintEditor->OnRenderGridShouldHideUIChanged().AddSP(this, &SRenderGridJob::Refresh);
	InBlueprintEditor->OnRenderGridJobsSelectionChanged().AddSP(this, &SRenderGridJob::Refresh);

	ChildSlot
	[
		DetailsView->AsShared()
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridJob::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) {}

void UE::RenderGrid::Private::SRenderGridJob::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		BlueprintEditor->MarkAsModified();
		BlueprintEditor->OnRenderGridChanged().Broadcast();
	}
}

void UE::RenderGrid::Private::SRenderGridJob::Refresh()
{
	if (DetailsView.IsValid())
	{
		if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
		{
			TArray<TWeakObjectPtr<UObject>> WeakSelectedJobs;
			if (!BlueprintEditor->ShouldHideUI())
			{
				if (const TArray<URenderGridJob*> SelectedJobs = BlueprintEditor->GetSelectedRenderGridJobs(); (SelectedJobs.Num() == 1))
				{
					WeakSelectedJobs.Add(SelectedJobs[0]);
				}
			}
			DetailsView->SetObjects(WeakSelectedJobs);
		}
	}
}


#undef LOCTEXT_NAMESPACE
