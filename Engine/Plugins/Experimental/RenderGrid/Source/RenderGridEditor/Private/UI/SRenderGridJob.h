// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"


class IDetailsView;
class URenderGridJob;
class URenderGridQueue;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * A widget with which the user can modify the selected render grid job.
	 * Can only modify 1 render grid job at a time, this widget will show nothing when 0 or 2+ render grid jobs are selected.
	 */
	class SRenderGridJob : public SCompoundWidget, public FNotifyHook
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridJob) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

		// FNotifyHook interface
		virtual void NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) override;
		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;
		// End of FNotifyHook interface

	private:
		/** Updates the details view. */
		void Refresh();

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** A reference to the details view. */
		TSharedPtr<IDetailsView> DetailsView;
	};
}
