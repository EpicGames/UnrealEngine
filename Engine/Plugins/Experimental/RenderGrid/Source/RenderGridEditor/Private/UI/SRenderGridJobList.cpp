// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridJobList.h"

#include "Commands/RenderGridEditorCommands.h"
#include "UI/Components/SRenderGridDragHandle.h"
#include "UI/Components/SRenderGridEditableTextBlock.h"
#include "UI/Components/SRenderGridFileSelectorTextBlock.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridQueue.h"
#include "IRenderGridEditor.h"
#include "Utils/RenderGridUtils.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineQueue.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "Blueprints/RenderGridBlueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "LevelSequence.h"

#define LOCTEXT_NAMESPACE "SRenderGridJobList"


namespace UE::RenderGrid::Private::FRenderGridJobListColumns
{
	const FName DragDropHandle = TEXT("DragDropHandle");
	const FName IsEnabled = TEXT("IsEnabled");
	const FName JobId = TEXT("JobId");
	const FName JobName = TEXT("JobName");
	const FName OutputDirectory = TEXT("OutputDirectory");
	const FName RenderPreset = TEXT("RenderPreset");
	const FName StartFrame = TEXT("StartFrame");
	const FName EndFrame = TEXT("EndFrame");
	const FName Resolution = TEXT("Resolution");
	const FName Tags = TEXT("Tags");
	const FName Duration = TEXT("Duration");
	const FName RenderingStatus = TEXT("Status");
}


void UE::RenderGrid::Private::SRenderGridJobList::Tick(const FGeometry&, const double, const float)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if ((RenderGridWeakPtr != BlueprintEditor->GetInstance()) || (BlueprintEditor->IsBatchRendering() != bPreviousIsBatchRendering))
		{
			Refresh();
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridJobList::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	TSharedPtr<SWidget> BaseThis = AsShared();
	TSharedPtr<SRenderGridJobList> This = StaticCastSharedPtr<SRenderGridJobList>(BaseThis);

	BlueprintEditorWeakPtr = InBlueprintEditor;

	Refresh();
	InBlueprintEditor->OnRenderGridJobCreated().AddSP(this, &SRenderGridJobList::OnRenderGridJobCreated);
	InBlueprintEditor->OnRenderGridChanged().AddSP(this, &SRenderGridJobList::Refresh);
	InBlueprintEditor->OnRenderGridJobsSelectionChanged().AddSP(this, &SRenderGridJobList::Refresh);
	FCoreUObjectDelegates::OnObjectModified.AddSP(this, &SRenderGridJobList::OnObjectModified);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SNew(SHorizontalBox)

			// Search Box
			+ SHorizontalBox::Slot()
			.Padding(4.f, 2.f)
			[
				SAssignNew(RenderGridSearchBox, SSearchBox)
				.HintText(LOCTEXT("Search_HintText", "Search Tags | Text"))
				.OnTextChanged(this, &SRenderGridJobList::OnSearchBarTextChanged)
			]

			/* // Filters
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 2.f, 2.f, 2.f)
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
				.ForegroundColor(FLinearColor::White)
				.ToolTipText(LOCTEXT("Filters_Tooltip", "Filter options for the Job List."))
				.HasDownArrow(true)
				.ContentPadding(0.f)
				.ButtonContent()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.12"))
					.Text(FText::FromString(FString(TEXT("\xf0b0")))) // fa-filter
				]
			] */
		]

		// Job List
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.f)
			[
				SAssignNew(RenderGridJobListWidget, SRenderGridJobListTable, BlueprintEditorWeakPtr, This)
				.ListItemsSource(&RenderGridJobs)
				.HeaderRow(
					SNew(SHeaderRow)

					+ SHeaderRow::Column(FRenderGridJobListColumns::DragDropHandle)
					.DefaultLabel(FText())
					.FixedWidth(36.0f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::IsEnabled)
					.DefaultLabel(LOCTEXT("JobListIsEnabledColumnHeader", "Enabled"))
					.FixedWidth(30.0f) //55.0f for text : "Enabled"
					[
						SAssignNew(RenderGridJobEnabledHeaderCheckbox, SCheckBox)
						.IsChecked(true)
						.OnCheckStateChanged(this, &SRenderGridJobList::OnHeaderCheckboxToggled)
					]

					+ SHeaderRow::Column(FRenderGridJobListColumns::JobId)
					.DefaultLabel(LOCTEXT("JobListIDColumnHeader", "Job ID"))
					.FillWidth(0.3f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::JobName)
					.DefaultLabel(LOCTEXT("JobListNameColumnHeader", "Job Name"))
					.FillWidth(0.3f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::OutputDirectory)
					.DefaultLabel(LOCTEXT("JobListOutDirColumnHeader", "Output Directory"))
					.FillWidth(0.7f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::RenderPreset)
					.DefaultLabel(LOCTEXT("JobListRenderPresetColumnHeader", "Render Preset"))
					.FillWidth(0.5f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::StartFrame)
					.DefaultLabel(LOCTEXT("JobListStartFrameColumnHeader", "Start Frame"))
					.FixedWidth(80.0f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::EndFrame)
					.DefaultLabel(LOCTEXT("JobListEndFrameColumnHeader", "End Frame"))
					.FixedWidth(80.0f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::Resolution)
					.DefaultLabel(LOCTEXT("JobListResolutionColumnHeader", "Resolution"))
					.FillWidth(0.7f)

					+ SHeaderRow::Column(FRenderGridJobListColumns::Duration)
					.DefaultLabel(LOCTEXT("JobListEstDurColumnHeader", "Est Duration"))
					.FixedWidth(120.0f)
				)
			]
		]
	];

	Refresh();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply UE::RenderGrid::Private::SRenderGridJobList::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
		{
			FMenuBuilder MenuBuilder(true, BlueprintEditor->GetToolkitCommands());
			{
				MenuBuilder.BeginSection("RenderGridJobListRowContextMenuSection", LOCTEXT("RenderGridJobListRowContextMenuHeading", "Options"));
				{
					MenuBuilder.AddMenuEntry(
						FRenderGridEditorCommands::Get().AddJob,
						NAME_None,
						TAttribute<FText>(),
						TAttribute<FText>(),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
					);

					if (!BlueprintEditor->GetSelectedRenderGridJobs().IsEmpty())
					{
						MenuBuilder.AddMenuEntry(
							FRenderGridEditorCommands::Get().DuplicateJob,
							NAME_None,
							TAttribute<FText>(),
							TAttribute<FText>(),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate")
						);

						MenuBuilder.AddMenuEntry(
							FRenderGridEditorCommands::Get().DeleteJob,
							NAME_None,
							TAttribute<FText>(),
							TAttribute<FText>(),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete")
						);
					}
				}
				MenuBuilder.EndSection();
			}
			TSharedPtr<SWidget> MenuContent = MenuBuilder.MakeWidget();

			if (MenuContent.IsValid() && (MouseEvent.GetEventPath() != nullptr))
			{
				FWidgetPath WidgetPath = *MouseEvent.GetEventPath();
				if (FSlateApplication::IsInitialized())
				{
					FSlateApplication::Get().PushMenu(WidgetPath.Widgets.Last().Widget, WidgetPath, MenuContent.ToSharedRef(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
				}
			}
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void UE::RenderGrid::Private::SRenderGridJobList::OnRenderGridJobCreated(URenderGridJob* Job)
{
	if (!RenderGridJobEnabledHeaderCheckbox.IsValid())
	{
		return;
	}
	Job->SetIsEnabled(RenderGridJobEnabledHeaderCheckbox->GetCheckedState() != ECheckBoxState::Unchecked);
}

void UE::RenderGrid::Private::SRenderGridJobList::OnHeaderCheckboxToggled(ECheckBoxState State)
{
	bool bRefresh = false;

	for (URenderGridJob* Job : RenderGridJobs)
	{
		Job->SetIsEnabled(State == ECheckBoxState::Checked);
		bRefresh = true;
	}

	if (bRefresh)
	{
		Refresh();
	}
}

ECheckBoxState UE::RenderGrid::Private::SRenderGridJobList::GetDesiredHeaderEnabledCheckboxState()
{
	ECheckBoxState State = ECheckBoxState::Checked;
	bool bFirstJob = true;
	for (URenderGridJob* Job : RenderGridJobs)
	{
		ECheckBoxState JobState = (Job->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
		if (bFirstJob)
		{
			State = JobState;
			bFirstJob = false;
		}
		else if (State != JobState)
		{
			return ECheckBoxState::Undetermined;
		}
	}
	return State;
}

void UE::RenderGrid::Private::SRenderGridJobList::AddRenderStatusColumn()
{
	if (!RenderGridJobListWidget.IsValid())
	{
		return;
	}
	RenderGridJobListWidget->GetHeaderRow()->AddColumn(SHeaderRow::Column(FRenderGridJobListColumns::RenderingStatus)
		.DefaultLabel(LOCTEXT("JobListRenderStatusColumnHeader", "Render Status"))
		.FixedWidth(110.0f));
}

void UE::RenderGrid::Private::SRenderGridJobList::RemoveRenderStatusColumn()
{
	if (!RenderGridJobListWidget.IsValid())
	{
		return;
	}
	RenderGridJobListWidget->GetHeaderRow()->RemoveColumn(FRenderGridJobListColumns::RenderingStatus);
}

void UE::RenderGrid::Private::SRenderGridJobList::OnObjectModified(UObject* Object)
{
	if (Cast<UMoviePipelineOutputSetting>(Object) || Cast<UMovieSceneSequence>(Object) || Cast<UMovieScene>(Object) || Cast<UMovieSceneTrack>(Object) || Cast<UMovieSceneSection>(Object) || Cast<UMovieSceneSubTrack>(Object) || Cast<UMovieSceneSubSection>(Object))
	{
		Refresh();
	}
}

void UE::RenderGrid::Private::SRenderGridJobList::Refresh()
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		bPreviousIsBatchRendering = BlueprintEditor->IsBatchRendering(); // show all jobs during a batch render, ignore the search bar
		const FString SearchBarContent = (!RenderGridSearchBox.IsValid() ? TEXT("") : RenderGridSearchBox->GetText().ToString());

		RenderGridJobs.Empty();
		RenderGridWeakPtr = BlueprintEditor->GetInstance();
		if (URenderGrid* Grid = RenderGridWeakPtr.Get(); IsValid(Grid))
		{
			for (URenderGridJob* Job : Grid->GetRenderGridJobs())
			{
				if (bPreviousIsBatchRendering || Job->MatchesSearchTerm(SearchBarContent))
				{
					RenderGridJobs.Add(Job);
				}
			}
		}

		RefreshHeaderEnabledCheckbox();

		RemoveRenderStatusColumn();
		if (bPreviousIsBatchRendering)
		{
			AddRenderStatusColumn();
		}

		if (RenderGridJobListWidget.IsValid())
		{
			RenderGridJobListWidget->RebuildList(); // rebuild is needed (instead of using RequestListRefresh()), because otherwise it won't show the changes made to the URenderGridJob variables

			TArray<URenderGridJob*> SelectedJobs = BlueprintEditor->GetSelectedRenderGridJobs().FilterByPredicate([this](URenderGridJob* Job)
			{
				return IsValid(Job) && RenderGridJobs.Contains(Job);
			});
			RenderGridJobListWidget->ClearSelection();
			RenderGridJobListWidget->SetItemSelection(SelectedJobs, true);
		}
	}
}

void UE::RenderGrid::Private::SRenderGridJobList::RefreshHeaderEnabledCheckbox()
{
	if (!RenderGridJobEnabledHeaderCheckbox.IsValid())
	{
		return;
	}
	RenderGridJobEnabledHeaderCheckbox->SetIsChecked(GetDesiredHeaderEnabledCheckboxState());
}


void UE::RenderGrid::Private::SRenderGridJobListTable::Construct(const FArguments& InArgs, TWeakPtr<IRenderGridEditor> InBlueprintEditor, const TSharedPtr<SRenderGridJobList>& InJobListWidget)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	JobListWidget = InJobListWidget;

	SListView::Construct(
		SListView::FArguments()
		.ListItemsSource(InArgs._ListItemsSource)
		.HeaderRow(InArgs._HeaderRow)
		.OnGenerateRow(this, &SRenderGridJobListTable::HandleJobListGenerateRow)
		.OnSelectionChanged(this, &SRenderGridJobListTable::HandleJobListSelectionChanged)
		.SelectionMode(ESelectionMode::Multi)
		.ClearSelectionOnClick(false)
	);
}

FReply UE::RenderGrid::Private::SRenderGridJobListTable::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return JobListWidget->OnMouseButtonUp(MyGeometry, MouseEvent);
	}
	return SListView<URenderGridJob*>::OnMouseButtonUp(MyGeometry, MouseEvent);
}

TSharedRef<ITableRow> UE::RenderGrid::Private::SRenderGridJobListTable::HandleJobListGenerateRow(URenderGridJob* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SRenderGridJobListTableRow, OwnerTable, BlueprintEditorWeakPtr, Item, JobListWidget);
}

void UE::RenderGrid::Private::SRenderGridJobListTable::HandleJobListSelectionChanged(URenderGridJob* Item, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Type::Direct)
	{
		return;
	}
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeJobSelection", "Change Job Selection"));
		BlueprintEditor->SetSelectedRenderGridJobs(GetSelectedItems());
	}
}


void UE::RenderGrid::Private::SRenderGridJobListTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<IRenderGridEditor> InBlueprintEditor, URenderGridJob* InRenderGridJob, const TSharedPtr<SRenderGridJobList>& InJobListWidget)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	RenderGridJobWeakPtr = InRenderGridJob;
	JobListWidget = InJobListWidget;

	SMultiColumnTableRow<URenderGridJob*>::Construct(FSuperRowType::FArguments()
		.OnCanAcceptDrop(this, &SRenderGridJobListTableRow::OnCanAcceptDrop)
		.OnAcceptDrop(this, &SRenderGridJobListTableRow::OnAcceptDrop),
		InOwnerTableView);
}

FReply UE::RenderGrid::Private::SRenderGridJobListTableRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return JobListWidget->OnMouseButtonUp(MyGeometry, MouseEvent);
	}
	return SMultiColumnTableRow<URenderGridJob*>::OnMouseButtonUp(MyGeometry, MouseEvent);
}

TOptional<EItemDropZone> UE::RenderGrid::Private::SRenderGridJobListTableRow::OnCanAcceptDrop(const FDragDropEvent& InEvent, EItemDropZone InItemDropZone, URenderGridJob* InJob)
{
	if (BlueprintEditorWeakPtr.IsValid() && InEvent.GetOperationAs<FRenderGridJobListTableRowDragDropOp>())
	{
		if (InItemDropZone == EItemDropZone::OntoItem)
		{
			return EItemDropZone::BelowItem;
		}
		return InItemDropZone;
	}
	return TOptional<EItemDropZone>();
}

FReply UE::RenderGrid::Private::SRenderGridJobListTableRow::OnAcceptDrop(const FDragDropEvent& InEvent, EItemDropZone InItemDropZone, URenderGridJob* InJob)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (const TSharedPtr<FRenderGridJobListTableRowDragDropOp> DragDropOp = InEvent.GetOperationAs<FRenderGridJobListTableRowDragDropOp>())
		{
			if (URenderGrid* Instance = BlueprintEditor->GetInstance(); IsValid(Instance))
			{
				FScopedTransaction Transaction(LOCTEXT("MoveJob", "Move Job"));
				BlueprintEditor->MarkAsModified();
				if (Instance->ReorderRenderGridJob(DragDropOp->GetJob(), InJob, (InItemDropZone != EItemDropZone::AboveItem)))
				{
					BlueprintEditor->GetRenderGridBlueprint()->PropagateJobsToAsset(Instance);
					BlueprintEditor->OnRenderGridChanged().Broadcast();
					return FReply::Handled();
				}
			}
		}
	}
	return FReply::Unhandled();
}

TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridJobListTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (URenderGridJob* Job = RenderGridJobWeakPtr.Get(); IsValid(Job))
	{
		if (ColumnName == FRenderGridJobListColumns::DragDropHandle)
		{
			return SNew(SBox)
				.Padding(FMargin(0.0f, 2.0f, 2.0f, 2.0f))
				[
					SNew(SScaleBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Stretch(EStretch::ScaleToFit)
					.StretchDirection(EStretchDirection::Both)
					[
						SNew(SRenderGridDragHandle<FRenderGridJobListTableRowDragDropOp>, Job)
						.Widget(SharedThis(this))
					]
				];
		}
		else if (ColumnName == FRenderGridJobListColumns::IsEnabled)
		{
			return SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(Job->GetIsEnabled())
					.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
					{
						if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
						{
							{
								FScopedTransaction Transaction(LOCTEXT("ChangeJobId", "Change Job Id"));
								if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
								{
									BlueprintEditor->MarkAsModified();
								}
								RenderGridJob->Modify();

								RenderGridJob->SetIsEnabled(State == ECheckBoxState::Checked);
							}

							if (JobListWidget.IsValid())
							{
								JobListWidget->RefreshHeaderEnabledCheckbox();
							}
						}
					})
				];
		}
		else if (ColumnName == FRenderGridJobListColumns::JobId)
		{
			return SNew(SRenderGridEditableTextBlock)
				.Text(FText::FromString(Job->GetJobId()))
				.OnTextCommitted_Lambda([this](const FText& InLabel, ETextCommit::Type InCommitInfo) -> FText
				{
					if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
					{
						const FString OldJobId = RenderGridJob->GetJobId();
						const FString NewJobId = FRenderGridUtils::PurgeJobIdOrReturnEmptyString(InLabel.ToString());
						if (NewJobId.IsEmpty() || (RenderGridJob->GetJobId() == NewJobId))
						{
							return FText::FromString(OldJobId);
						}

						if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
						{
							if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
							{
								if (Grid->DoesJobIdExist(NewJobId))
								{
									FMessageDialog::Open(
										EAppMsgType::Ok,
										FText::Format(LOCTEXT("JobIdNotUniqueMessage", "Job ID \"{0}\" is not unique."), FText::FromString(NewJobId)),
										LOCTEXT("JobIdNotUniqueTitle", "Duplicate Job IDs"));
									return FText::FromString(OldJobId);
								}

								FScopedTransaction Transaction(LOCTEXT("ChangeJobId", "Change Job Id"));
								BlueprintEditor->MarkAsModified();
								RenderGridJob->Modify();

								RenderGridJob->SetJobId(NewJobId);
								return FText::FromString(RenderGridJob->GetJobId());
							}
						}
						return FText::FromString(OldJobId);
					}
					return FText();
				});
		}
		else if (ColumnName == FRenderGridJobListColumns::JobName)
		{
			return SNew(SRenderGridEditableTextBlock)
				.Text(FText::FromString(Job->GetJobName()))
				.OnTextCommitted_Lambda([this](const FText& InLabel, ETextCommit::Type InCommitInfo) -> FText
				{
					if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
					{
						FScopedTransaction Transaction(LOCTEXT("ChangeJobName", "Change Job Name"));
						if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
						{
							BlueprintEditor->MarkAsModified();
						}
						RenderGridJob->Modify();

						RenderGridJob->SetJobName(InLabel.ToString());
						return FText::FromString(RenderGridJob->GetJobName());
					}
					return FText();
				});
		}
		else if (ColumnName == FRenderGridJobListColumns::OutputDirectory)
		{
			return SNew(SRenderGridFileSelectorTextBlock)
				.Text(FText::FromString(Job->GetOutputDirectoryForDisplay()))
				.FolderPath_Lambda([this]() -> FString
				{
					if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
					{
						return RenderGridJob->GetOutputDirectory();
					}
					return FString();
				})
				.OnTextCommitted_Lambda([this](const FText& InLabel, ETextCommit::Type InCommitInfo) -> FText
				{
					if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
					{
						FScopedTransaction Transaction(LOCTEXT("ChangeJobOutputDirectory", "Change Job Output Directory"));
						if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
						{
							BlueprintEditor->MarkAsModified();
						}
						RenderGridJob->Modify();

						RenderGridJob->SetOutputDirectory(InLabel.ToString());
						return FText::FromString(RenderGridJob->GetOutputDirectoryForDisplay());
					}
					return FText();
				});
		}
		else if (ColumnName == FRenderGridJobListColumns::RenderPreset)
		{
			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(UMoviePipelinePrimaryConfig::StaticClass())
				.ObjectPath_Lambda([this]() -> FString
				{
					if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
					{
						if (UMoviePipelinePrimaryConfig* Preset = RenderGridJob->GetRenderPreset(); IsValid(Preset))
						{
							return Preset->GetPathName();
						}
					}
					return FString();
				})
				.OnObjectChanged_Lambda([this](const FAssetData& AssetData) -> void
				{
					if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
					{
						{
							FScopedTransaction Transaction(LOCTEXT("ChangeJobRenderPreset", "Change Job Render Preset"));
							if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
							{
								BlueprintEditor->MarkAsModified();
							}
							RenderGridJob->Modify();

							RenderGridJob->SetRenderPreset(nullptr);
							if (UObject* AssetDataAsset = AssetData.GetAsset(); IsValid(AssetDataAsset))
							{
								if (UMoviePipelinePrimaryConfig* Preset = Cast<UMoviePipelinePrimaryConfig>(AssetDataAsset))
								{
									RenderGridJob->SetRenderPreset(Preset);
								}
							}
						}

						if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
						{
							BlueprintEditor->OnRenderGridChanged().Broadcast();
						}
					}
				})
				.AllowClear(true)
				.DisplayUseSelected(true)
				.DisplayBrowse(true)
				.DisplayThumbnail(false);
		}
		else if ((ColumnName == FRenderGridJobListColumns::StartFrame) || (ColumnName == FRenderGridJobListColumns::EndFrame))
		{
			FText Text;
			if (TOptional<int32> Frame = ((ColumnName == FRenderGridJobListColumns::StartFrame) ? Job->GetStartFrame() : Job->GetEndFrame()))
			{
				Text = FText::AsNumber(*Frame);
			}
			return SNew(SBox)
				.VAlign(VAlign_Center)
				//.HAlign(HAlign_Right)
				.Padding(FMargin(10, 0))
				[
					SNew(STextBlock).Text(Text)
				];
		}
		else if (ColumnName == FRenderGridJobListColumns::Tags)
		{
			//TODO: add support for tags
		}
		else if (ColumnName == FRenderGridJobListColumns::Resolution)
		{
			FIntPoint Resolution = Job->GetOutputResolution();

			FNumberFormattingOptions NumberFormattingOptions;
			NumberFormattingOptions.UseGrouping = false;

			FText ResolutionFormat = LOCTEXT("Resolution", "{Width} x {Height}");
			FFormatNamedArguments ResolutionArguments;
			ResolutionArguments.Add(TEXT("Width"), FText::AsNumber(Resolution.X, &NumberFormattingOptions));
			ResolutionArguments.Add(TEXT("Height"), FText::AsNumber(Resolution.Y, &NumberFormattingOptions));
			FText Text = FText::Format(ResolutionFormat, ResolutionArguments);

			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(10, 0))
				[
					SNew(STextBlock).Text(Text)
				];
		}
		else if (ColumnName == FRenderGridJobListColumns::Duration)
		{
			FText Text;
			if (TOptional<double> Duration = Job->GetDuration())
			{
				FTimespan Timespan = FTimespan::FromSeconds(*Duration);
				int32 Hours = static_cast<int32>(Timespan.GetTotalHours());
				int32 Minutes = Timespan.GetMinutes();
				int32 Seconds = Timespan.GetSeconds();

				FNumberFormattingOptions NumberFormattingOptions;
				NumberFormattingOptions.MinimumIntegralDigits = 2;
				NumberFormattingOptions.MaximumIntegralDigits = 2;

				FText TimespanFormat = NSLOCTEXT("Timespan", "Format_HoursMinutesSeconds", "{Hours}:{Minutes}:{Seconds}");
				FFormatNamedArguments TimeArguments;
				TimeArguments.Add(TEXT("Hours"), Hours);
				TimeArguments.Add(TEXT("Minutes"), FText::AsNumber(Minutes, &NumberFormattingOptions));
				TimeArguments.Add(TEXT("Seconds"), FText::AsNumber(Seconds, &NumberFormattingOptions));
				Text = FText::Format(TimespanFormat, TimeArguments);
			}
			return SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text(Text)
				];
		}
		else if (ColumnName == FRenderGridJobListColumns::RenderingStatus)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() -> FText
					{
						return GetRenderStatusText();
					})
				];
		}
	}
	return SNullWidget::NullWidget;
}

FText UE::RenderGrid::Private::SRenderGridJobListTableRow::GetRenderStatusText() const
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGridQueue* RenderQueue = BlueprintEditor->GetBatchRenderQueue(); IsValid(RenderQueue))
		{
			if (URenderGridJob* RenderGridJob = RenderGridJobWeakPtr.Get(); IsValid(RenderGridJob))
			{
				return FText::FromString(RenderQueue->GetJobStatus(RenderGridJob));
			}
		}
	}
	return FText();
}


#undef LOCTEXT_NAMESPACE
