// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "RenderGrid/RenderGrid.h"
#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"


class SCheckBox;
class SSearchBox;
class URenderGridJob;
class URenderGridQueue;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}

namespace UE::RenderGrid::Private
{
	class SRenderGridJobListTable;
}


namespace UE::RenderGrid::Private
{
	DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnRenderGridJobListEditableTextBlockTextCommitted, const FText&, ETextCommit::Type);


	/**
	 * A widget with which the user can see and modify the list of render grid jobs that the render grid contains.
	 */
	class SRenderGridJobList : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridJobList) {}
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

		//~ Begin SWidget Interface
		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		//~ End SWidget Interface

		/** Refreshes the content of this widget. */
		void Refresh();

		/** Refreshes the state of the header is-job-enabled checkbox. */
		void RefreshHeaderEnabledCheckbox();

	private:
		/** Gets called when a render grid job is created. */
		void OnRenderGridJobCreated(URenderGridJob* Job);

		/** Gets called when the header is-job-enabled checkbox is toggled. */
		void OnHeaderCheckboxToggled(ECheckBoxState State);

		/** Gets the desired state of the header is-job-enabled checkbox. */
		ECheckBoxState GetDesiredHeaderEnabledCheckboxState();

		/** Adds the render status column to the job list. */
		void AddRenderStatusColumn();

		/** Removes the render status column from the job list. */
		void RemoveRenderStatusColumn();

	private:
		void OnObjectModified(UObject* Object);
		void OnSearchBarTextChanged(const FText& Text) { Refresh(); }

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The render grid that is shown in the UI. */
		TWeakObjectPtr<URenderGrid> RenderGridWeakPtr;

		/** The render grid jobs that are shown in the UI. */
		TArray<URenderGridJob*> RenderGridJobs;

		/** The widget that lists the render grid jobs. */
		TSharedPtr<SRenderGridJobListTable> RenderGridJobListWidget;

		/** The search bar widget. */
		TSharedPtr<SSearchBox> RenderGridSearchBox;

		/** The header checkbox for the enable/disable column. */
		TSharedPtr<SCheckBox> RenderGridJobEnabledHeaderCheckbox;

		/** The value of BlueprintEditorWeakPtr.Get()->IsBatchRendering() during the last Refresh(). */
		bool bPreviousIsBatchRendering;
	};


	/**
	 * The widget that contains the rows of render grid jobs.
	 */
	class SRenderGridJobListTable : public SListView<URenderGridJob*>
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridJobListTable) {}
			SLATE_ARGUMENT(const TArray<URenderGridJob*>*, ListItemsSource)
			SLATE_ARGUMENT(TSharedPtr<SHeaderRow>, HeaderRow)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakPtr<IRenderGridEditor> InBlueprintEditor, const TSharedPtr<SRenderGridJobList>& InJobListWidget);

		//~ Begin SWidget Interface
		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		//~ End SWidget Interface

	private:
		/** Callback for generating a row widget in the session tree view. */
		TSharedRef<ITableRow> HandleJobListGenerateRow(URenderGridJob* Item, const TSharedRef<STableViewBase>& OwnerTable);

		/** Callback for session tree view selection changes. */
		void HandleJobListSelectionChanged(URenderGridJob* Item, ESelectInfo::Type SelectInfo);

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** A reference to the job list (the parent widget). */
		TSharedPtr<SRenderGridJobList> JobListWidget;
	};


	/**
	 * The widget that represents a single render grid job (a single row).
	 */
	class SRenderGridJobListTableRow : public SMultiColumnTableRow<URenderGridJob*>
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridJobListTableRow) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<IRenderGridEditor> InBlueprintEditor, URenderGridJob* InRenderGridJob, const TSharedPtr<SRenderGridJobList>& InJobListWidget);

		//~ Begin SWidget Interface
		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		//~ End SWidget Interface

		TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InEvent, EItemDropZone InItemDropZone, URenderGridJob* InJob);
		FReply OnAcceptDrop(const FDragDropEvent& InEvent, EItemDropZone InItemDropZone, URenderGridJob* InJob);
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

		FText GetRenderStatusText() const;

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** A reference to the render grid job instance. */
		TWeakObjectPtr<URenderGridJob> RenderGridJobWeakPtr;

		/** A reference to the job list (the parent widget). */
		TSharedPtr<SRenderGridJobList> JobListWidget;
	};


	/**
	 * The class that makes it possible to drag and drop render grid jobs (allowing the user to reorganize the render grid job list).
	 */
	class FRenderGridJobListTableRowDragDropOp final : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FRenderGridJobListTableRowDragDropOp, FDragDropOperation)

		using WidgetType = SRenderGridJobListTableRow;
		using HeldItemType = TObjectPtr<URenderGridJob>;

		FRenderGridJobListTableRowDragDropOp(const TSharedPtr<WidgetType> InWidget, const HeldItemType InJob)
			: Job(InJob)
		{
			DecoratorWidget = SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Border"))
				.Content()
				[
					InWidget.ToSharedRef()
				];
		}

		HeldItemType GetJob() const
		{
			return Job;
		}

		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return DecoratorWidget;
		}

	private:
		/** The held item. */
		HeldItemType Job;

		/** Holds the displayed widget. */
		TSharedPtr<SWidget> DecoratorWidget;
	};
}
