// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridProps.h"
#include "UI/SRenderGridPropsBase.h"
#include "RenderGrid/RenderGrid.h"
#include "IRenderGridEditor.h"
#include "IRenderGridEditorModule.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "SRenderGridProps"


void UE::RenderGrid::Private::SRenderGridProps::Tick(const FGeometry&, const double, const float)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (WidgetPropsSourceWeakPtr != Grid->GetPropsSource())
			{
				Refresh();
			}
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridProps::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;

	SAssignNew(WidgetContainer, SBorder)
		.Padding(8.0f)
		.BorderImage(new FSlateNoResource());

	Refresh();
	//InBlueprintEditor->OnRenderGridChanged().AddSP(this, &SRenderGridProps::Refresh); // should technically be correct, but causes the remote control properties to bug out
	InBlueprintEditor->OnRenderGridPreviewRenderedFirstTimeSinceAppStart().AddSP(this, &SRenderGridProps::Refresh);
	InBlueprintEditor->OnRenderGridShouldHideUIChanged().AddSP(this, &SRenderGridProps::Refresh);
	InBlueprintEditor->OnRenderGridJobsSelectionChanged().AddSP(this, &SRenderGridProps::Refresh);

	ChildSlot
	[
		WidgetContainer.ToSharedRef()
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridProps::Refresh()
{
	if (!WidgetContainer.IsValid())
	{
		return;
	}
	WidgetContainer->ClearContent();

	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (!BlueprintEditor->ShouldHideUI())
		{
			if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
			{
				WidgetPropsSourceWeakPtr = Grid->GetPropsSource();
				if (URenderGridPropsSourceBase* WidgetPropsSource = WidgetPropsSourceWeakPtr.Get(); IsValid(WidgetPropsSource))
				{
					if (const TSharedPtr<SRenderGridPropsBase> Widget = IRenderGridEditorModule::Get().CreatePropsSourceWidget(WidgetPropsSource, BlueprintEditor))
					{
						WidgetContainer->SetContent(Widget.ToSharedRef());
					}
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
