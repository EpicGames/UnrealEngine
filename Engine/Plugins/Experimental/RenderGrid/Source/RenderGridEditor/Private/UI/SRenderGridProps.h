// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


class SBorder;
class URenderGridJob;
class URenderGridPropsSourceBase;
class URenderGridQueue;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * A widget with which the user can modify the props (like the remote control field values) of the selected render grid job.
	 * Can only modify the props of 1 render grid job at a time, this widget will show nothing when 0 or 2+ render grid jobs are selected.
	 */
	class SRenderGridProps : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridProps) {}
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

		/** Refreshes the content of this widget. */
		void Refresh();

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The widget that lists the property rows. */
		TSharedPtr<SBorder> WidgetContainer;

		/** The props source that's being shown in this widget. */
		TWeakObjectPtr<URenderGridPropsSourceBase> WidgetPropsSourceWeakPtr;
	};
}
