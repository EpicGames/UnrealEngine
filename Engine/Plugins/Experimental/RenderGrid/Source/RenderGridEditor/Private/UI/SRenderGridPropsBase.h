// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


namespace UE::RenderGrid::Private
{
	/**
	 * The base class of the render grid props widget.
	 */
	class SRenderGridPropsBase : public SCompoundWidget
	{};
}
