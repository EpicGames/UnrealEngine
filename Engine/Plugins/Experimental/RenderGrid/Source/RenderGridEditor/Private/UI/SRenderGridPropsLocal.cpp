// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridPropsLocal.h"
#include "RenderGrid/RenderGrid.h"
#include "IRenderGridEditor.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SRenderGridPropsLocal"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridPropsLocal::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor, URenderGridPropsSourceLocal* InPropsSource)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	PropsSourceWeakPtr = InPropsSource;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
