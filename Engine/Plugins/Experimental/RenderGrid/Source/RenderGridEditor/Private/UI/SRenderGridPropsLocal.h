// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UI/SRenderGridPropsBase.h"
#include "RenderGrid/RenderGridPropsSource.h"


namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The render grid props implementation for local properties.
	 */
	class SRenderGridPropsLocal : public SRenderGridPropsBase
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridPropsLocal) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor, URenderGridPropsSourceLocal* InPropsSource);

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The props source control. */
		TWeakObjectPtr<URenderGridPropsSourceLocal> PropsSourceWeakPtr;
	};
}
