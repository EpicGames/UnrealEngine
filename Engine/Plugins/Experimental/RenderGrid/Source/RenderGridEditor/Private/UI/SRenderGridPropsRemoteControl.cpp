// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridPropsRemoteControl.h"
#include "UI/Components/SRenderGridRemoteControlTreeNode.h"
#include "UI/Components/SRenderGridRemoteControlField.h"
#include "IRenderGridEditor.h"
#include "IRenderGridModule.h"
#include "ScopedTransaction.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridManager.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SRenderGridPropsRemoteControl"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridPropsRemoteControl::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor, URenderGridPropsSourceRemoteControl* InPropsSource)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	PropsSourceWeakPtr = InPropsSource;

	SAssignNew(RowWidgetsContainer, SVerticalBox);
	UpdateStoredValuesAndRefresh(true);

	InBlueprintEditor->OnRenderGridJobsSelectionChanged().AddSP(this, &SRenderGridPropsRemoteControl::OnRenderGridJobsSelectionChanged);
	if (IsValid(InPropsSource))
	{
		if (TObjectPtr<URemoteControlPreset> Preset = InPropsSource->GetProps()->GetRemoteControlPreset())
		{
			Preset->OnEntityExposed().AddSP(this, &SRenderGridPropsRemoteControl::OnRemoteControlEntitiesExposed);
			Preset->OnEntityUnexposed().AddSP(this, &SRenderGridPropsRemoteControl::OnRemoteControlEntitiesUnexposed);
			Preset->OnEntitiesUpdated().AddSP(this, &SRenderGridPropsRemoteControl::OnRemoteControlEntitiesUpdated);
			Preset->OnPresetLayoutModified().AddSP(this, &SRenderGridPropsRemoteControl::OnRemoteControlPresetLayoutModified);
			Preset->OnExposedPropertiesModified().AddSP(this, &SRenderGridPropsRemoteControl::OnRemoteControlExposedPropertiesModified);
		}
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(8.0f)
		.BorderImage(new FSlateNoResource())
		[
			RowWidgetsContainer.ToSharedRef()
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void UE::RenderGrid::Private::SRenderGridPropsRemoteControl::UpdateStoredValuesAndRefresh(const bool bForce)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		IRenderGridModule::Get().GetManager().UpdateStoredRenderGridJobsPropValues(BlueprintEditor->GetInstance());
		Refresh(bForce);
	}
}

void UE::RenderGrid::Private::SRenderGridPropsRemoteControl::Refresh(const bool bForce)
{
	if (!RowWidgetsContainer.IsValid())
	{
		return;
	}

	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		TArray<FRenderGridRemoteControlGenerateWidgetArgs> NewRowWidgetsArgs;
		if (URenderGridPropsSourceRemoteControl* PropsSource = PropsSourceWeakPtr.Get(); IsValid(PropsSource))
		{
			URenderGridPropsRemoteControl* Props = PropsSource->GetProps();
			for (URenderGridPropRemoteControl* Prop : Props->GetAllCasted())
			{
				TArray<uint8> PropData;
				if (!GetSelectedJobFieldValue(Prop->GetRemoteControlEntity(), PropData))
				{
					continue;
				}

				if (!BlueprintEditor->IsCurrentlyRenderingOrPlaying())
				{
					if (!Prop->SetValue(PropData))
					{
						continue;
					}
				}
				else if (!Prop->CanSetValue(PropData))
				{
					continue;
				}

				if (URemoteControlPreset* Preset = Props->GetRemoteControlPreset(); IsValid(Preset))
				{
					TSharedPtr<FRemoteControlField> EntityField = StaticCastSharedPtr<FRemoteControlField>(Prop->GetRemoteControlEntity());
					FRenderGridRemoteControlGenerateWidgetArgs Args;
					Args.Preset = Preset;
					Args.Entity = Prop->GetRemoteControlEntity();
					Args.EntityFieldLabel = EntityField.IsValid() ? EntityField->GetLabel() : FName();
					Args.ColumnSizeData.LeftColumnWidth = 0.3;
					Args.ColumnSizeData.RightColumnWidth = 0.7;
					NewRowWidgetsArgs.Add(Args);
				}
			}
		}

		if (bForce || (RowWidgetsArgs != NewRowWidgetsArgs))
		{
			RowWidgetsArgs = NewRowWidgetsArgs;
			RowWidgetsContainer->ClearChildren();
			RowWidgets.Empty();
			for (const FRenderGridRemoteControlGenerateWidgetArgs& RowWidgetArgs : RowWidgetsArgs)
			{
				TSharedPtr<SRenderGridRemoteControlTreeNode> RowWidget = SRenderGridRemoteControlField::MakeInstance(RowWidgetArgs);
				RowWidgets.Add(RowWidget);
				RowWidgetsContainer->AddSlot()
					.Padding(0.0f)
					.AutoHeight()
					[
						RowWidget.ToSharedRef()
					];
			}
		}
		else
		{
			for (const TSharedPtr<SRenderGridRemoteControlTreeNode>& RowWidget : RowWidgets)
			{
				RowWidget->RefreshValue();
			}
		}
	}
}


void UE::RenderGrid::Private::SRenderGridPropsRemoteControl::OnRemoteControlExposedPropertiesModified(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedProperties)
{
	if (!IsValid(Preset))
	{
		return;
	}
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (BlueprintEditor->IsCurrentlyRenderingOrPlaying())
		{
			return;
		}

		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (Grid->IsCurrentlyExecutingUserCode())
			{
				return;
			}

			bool bModified = false;
			for (const FGuid& Id : ModifiedProperties)
			{
				if (const TSharedPtr<FRemoteControlEntity> Entity = Preset->GetExposedEntity<FRemoteControlEntity>(Id).Pin())
				{
					TArray<uint8> Bytes;
					if (!URenderGridPropRemoteControl::GetValueOfEntity(Entity, Bytes))
					{
						continue;
					}
					TArray<uint8> StoredBytes;
					if (!GetSelectedJobFieldValue(Entity, StoredBytes))
					{
						continue;
					}
					if (Bytes == StoredBytes)
					{
						continue;
					}

					if (!SetSelectedJobFieldValue(Entity, Bytes))
					{
						continue;
					}
					bModified = true;
				}
			}

			if (bModified)
			{
				BlueprintEditor->MarkAsModified();
				BlueprintEditor->OnRenderGridChanged().Broadcast();
				Refresh();
			}
		}
	}
}


URenderGridJob* UE::RenderGrid::Private::SRenderGridPropsRemoteControl::GetSelectedJob()
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (const TArray<URenderGridJob*> SelectedJobs = BlueprintEditor->GetSelectedRenderGridJobs(); (SelectedJobs.Num() == 1))
		{
			return SelectedJobs[0];
		}
	}
	return nullptr;
}

bool UE::RenderGrid::Private::SRenderGridPropsRemoteControl::GetSelectedJobFieldValue(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, TArray<uint8>& OutBytes)
{
	OutBytes.Empty();
	if (RemoteControlEntity.IsValid())
	{
		if (URenderGridJob* SelectedJob = GetSelectedJob(); IsValid(SelectedJob))
		{
			return SelectedJob->GetRemoteControlValueBytes(RemoteControlEntity, OutBytes);
		}
	}
	return false;
}

bool UE::RenderGrid::Private::SRenderGridPropsRemoteControl::SetSelectedJobFieldValue(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes)
{
	if (RemoteControlEntity.IsValid())
	{
		if (URenderGridJob* SelectedJob = GetSelectedJob(); IsValid(SelectedJob))
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeJobProperty", "Change Job Property"));
			SelectedJob->Modify();
			return SelectedJob->SetRemoteControlValueBytes(RemoteControlEntity, Bytes);
		}
	}
	return false;
}


#undef LOCTEXT_NAMESPACE
