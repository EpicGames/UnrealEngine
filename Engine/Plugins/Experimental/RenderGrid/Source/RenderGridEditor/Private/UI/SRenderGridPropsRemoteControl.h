// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RemoteControlPreset.h"
#include "RenderGrid/RenderGridPropsSource.h"
#include "UI/Components/SRenderGridRemoteControlTreeNode.h"
#include "UI/SRenderGridPropsBase.h"


class URenderGridJob;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The render grid props implementation for remote control fields.
	 */
	class SRenderGridPropsRemoteControl : public SRenderGridPropsBase
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridPropsRemoteControl) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor, URenderGridPropsSourceRemoteControl* InPropsSource);

		/** Obtain the latest prop values and refreshes the content of this widget. */
		void UpdateStoredValuesAndRefresh(const bool bForce = false);

		/** Refreshes the content of this widget. */
		void Refresh(const bool bForce = false);

	private:
		void OnRenderGridJobsSelectionChanged() { Refresh(true); }
		void OnRemoteControlEntitiesExposed(URemoteControlPreset* Preset, const FGuid& EntityId) { UpdateStoredValuesAndRefresh(true); }
		void OnRemoteControlEntitiesUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId) { UpdateStoredValuesAndRefresh(true); }
		void OnRemoteControlEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities) { UpdateStoredValuesAndRefresh(); }
		void OnRemoteControlPresetLayoutModified(URemoteControlPreset* Preset) { UpdateStoredValuesAndRefresh(true); }
		void OnRemoteControlExposedPropertiesModified(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedProperties);

	private:
		/** Returns the currently selected render grid job if 1 render grid job is currently selected, returns nullptr otherwise. */
		URenderGridJob* GetSelectedJob();

		/** Obtains the value (as bytes) of the given prop (the given remote control entity), returns true if it succeeded, returns false otherwise. */
		bool GetSelectedJobFieldValue(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, TArray<uint8>& OutBytes);

		/** Sets the value of the given prop (the given remote control entity) with the given value (as bytes), returns true if it succeeded, returns false otherwise. */
		bool SetSelectedJobFieldValue(const TSharedPtr<FRemoteControlEntity>& RemoteControlEntity, const TArray<uint8>& Bytes);

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The props source control. */
		TWeakObjectPtr<URenderGridPropsSourceRemoteControl> PropsSourceWeakPtr;

		/** The widget that lists the property rows. */
		TSharedPtr<SVerticalBox> RowWidgetsContainer;

		/** The current property rows, needed to be able to refresh them, as well as to prevent garbage collection. */
		TArray<TSharedPtr<SRenderGridRemoteControlTreeNode>> RowWidgets;

		/** The arguments that were used to create the current property rows, needed to not recreate the property rows unnecessarily. */
		TArray<FRenderGridRemoteControlGenerateWidgetArgs> RowWidgetsArgs;
	};
}
