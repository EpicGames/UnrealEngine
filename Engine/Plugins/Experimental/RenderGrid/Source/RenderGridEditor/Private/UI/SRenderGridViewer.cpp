// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridViewer.h"
#include "UI/SRenderGridViewerLive.h"
#include "UI/SRenderGridViewerPreview.h"
#include "UI/SRenderGridViewerRendered.h"
#include "IRenderGridEditor.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SRenderGridViewer"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewer::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	ViewerMode = ERenderGridViewerMode::Live;
	CachedViewerMode = ERenderGridViewerMode::None;

	SAssignNew(WidgetContainer, SBorder)
		.Padding(0.0f)
		.BorderImage(new FSlateNoResource());

	Refresh();
	InBlueprintEditor->OnRenderGridShouldHideUIChanged().AddSP(this, &SRenderGridViewer::Refresh);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.HeightOverride(27.5f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					CreateViewerModeButton(LOCTEXT("Viewport", "Viewport"), ERenderGridViewerMode::Live)
				]
				+ SHorizontalBox::Slot()
				[
					CreateViewerModeButton(LOCTEXT("Preview Frame", "Preview Frame"), ERenderGridViewerMode::Preview)
				]
				+ SHorizontalBox::Slot()
				[
					CreateViewerModeButton(LOCTEXT("Rendered", "Rendered"), ERenderGridViewerMode::Rendered)
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			WidgetContainer.ToSharedRef()
		]
	];
}

TSharedRef<SWidget> UE::RenderGrid::Private::SRenderGridViewer::CreateViewerModeButton(const FText& ButtonText, const ERenderGridViewerMode ButtonViewerMode)
{
	return SNew(SCheckBox)
		.Style(FAppStyle::Get(), "PlacementBrowser.Tab")
		.IsChecked_Lambda([this, ButtonViewerMode]() -> ECheckBoxState
		{
			return (ViewerMode == ButtonViewerMode) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([this, ButtonViewerMode](ECheckBoxState State)
		{
			if (State != ECheckBoxState::Checked)
			{
				return;
			}
			ViewerMode = ButtonViewerMode;
			Refresh();
		})
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(ButtonText)
			]
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewer::Refresh()
{
	if (!WidgetContainer.IsValid())
	{
		return;
	}
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		ERenderGridViewerMode CurrentViewerMode = (BlueprintEditor->ShouldHideUI() ? ERenderGridViewerMode::None : ViewerMode);
		if (CurrentViewerMode == CachedViewerMode)
		{
			return;
		}

		CachedViewerMode = CurrentViewerMode;
		WidgetContainer->ClearContent();

		switch (CachedViewerMode)
		{
			case ERenderGridViewerMode::Preview:
				WidgetContainer->SetContent(SNew(SRenderGridViewerPreview, BlueprintEditor));
				break;

			case ERenderGridViewerMode::Rendered:
				WidgetContainer->SetContent(SNew(SRenderGridViewerRendered, BlueprintEditor));
				break;

			case ERenderGridViewerMode::None:
				break;

			case ERenderGridViewerMode::Live:
			default:
				WidgetContainer->SetContent(SNew(SRenderGridViewerLive, BlueprintEditor));
				break;
		}
	}
}


#undef LOCTEXT_NAMESPACE
