// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


class SBorder;
class URenderGridJob;
class URenderGridQueue;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * An enum containing the different render grid viewer modes that are currently available in the render grid plugin.
	 */
	enum class ERenderGridViewerMode : uint8
	{
		Live,
		Preview,
		Rendered,
		None
	};

	/**
	 * The render grid viewer, allows the user to see the expected render output, directly in the editor.
	 */
	class SRenderGridViewer : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridViewer) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

	private:
		/** Creates a tab button for a viewer mode. */
		TSharedRef<SWidget> CreateViewerModeButton(const FText& ButtonText, const ERenderGridViewerMode ButtonViewerMode);

	public:
		/** Refreshes the content of this widget. */
		void Refresh();

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The widget that lists the viewers. */
		TSharedPtr<SBorder> WidgetContainer;

		/** The current viewer mode that should be shown in the UI. */
		ERenderGridViewerMode ViewerMode;

		/** The last viewer mode that was shown in the UI. */
		ERenderGridViewerMode CachedViewerMode;
	};
}
