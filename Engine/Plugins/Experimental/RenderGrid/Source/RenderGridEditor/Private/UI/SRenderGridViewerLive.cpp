// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridViewerLive.h"
#include "Camera/CameraComponent.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerStart.h"
#include "IRenderGridEditor.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "RenderGrid/RenderGrid.h"
#include "SlateOptMacros.h"
#include "Styles/RenderGridEditorStyle.h"
#include "UI/Components/SRenderGridViewerFrameSlider.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScaleBox.h"

#define LOCTEXT_NAMESPACE "SRenderGridViewerLive"


UE::RenderGrid::Private::FRenderGridEditorViewportClient::FRenderGridEditorViewportClient(FPreviewScene* PreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget)
	: FEditorViewportClient(nullptr, PreviewScene, InEditorViewportWidget)
{
	FOVAngle = 90;
	ViewFOV = 90;
	Invalidate();

	bDisableInput = true;
	SetGameView(true);
	SetRealtime(false); // we manually render every frame, because automatic rendering stops temporarily when you're dragging another widget (with the mouse)
}


void UE::RenderGrid::Private::SRenderGridEditorViewport::Tick(const FGeometry&, const double, const float)
{
	Render();
}

void UE::RenderGrid::Private::SRenderGridEditorViewport::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	ViewportClient = MakeShareable(new FRenderGridEditorViewportClient(nullptr, SharedThis(this)));
	LevelSequencePlayerWorld = nullptr;
	LevelSequencePlayerActor = nullptr;
	LevelSequencePlayer = nullptr;
	LevelSequence = nullptr;
	RenderGridJob = nullptr;
	LevelSequenceTime = 0.0f;
	bRenderedLastAttempt = false;

	SEditorViewport::Construct(SEditorViewport::FArguments());
}

UE::RenderGrid::Private::SRenderGridEditorViewport::~SRenderGridEditorViewport()
{
	DestroySequencePlayer();
	ViewportClient.Reset();
}

void UE::RenderGrid::Private::SRenderGridEditorViewport::Render()
{
	bRenderedLastAttempt = false;

	if (!ViewportClient.IsValid() || !IsValid(RenderGridJob))
	{
		return;
	}

	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (BlueprintEditor->IsCurrentlyRenderingOrPlaying())
		{
			return;
		}

		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (!Grid->GetLevel().IsValid() || (Grid->GetLevel().Get() != GetWorld()))
			{
				return;
			}

			if (ULevelSequencePlayer* SequencePlayer = GetSequencePlayer(); IsValid(SequencePlayer))
			{
				if (!Grid->HasRenderGridJob(RenderGridJob))
				{
					return;
				}
				bRenderedLastAttempt = true;

				BlueprintEditor->SetIsDebugging(true);

				SequencePlayer->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(LevelSequenceTime, EUpdatePositionMethod::Play)); // execute this every tick, in case any sequencer values get overwritten (by remote control props for example)
				Grid->BeginViewportRender(RenderGridJob);

				if (UCameraComponent* Camera = SequencePlayer->GetActiveCameraComponent(); IsValid(Camera))
				{
					ViewportClient->SetViewLocation(Camera->GetComponentLocation());
					ViewportClient->SetViewRotation(Camera->GetComponentRotation());
					ViewportClient->ViewFOV = Camera->FieldOfView;
				}
				else if (UWorld* World = GetWorld(); IsValid(World))
				{
					if (APlayerController* LocalPlayerController = World->GetFirstPlayerController(); IsValid(LocalPlayerController))
					{
						FVector ViewLocation;
						FRotator ViewRotation;
						LocalPlayerController->GetPlayerViewPoint(ViewLocation, ViewRotation);
						ViewportClient->SetViewLocation(ViewLocation);
						ViewportClient->SetViewRotation(ViewRotation);
						ViewportClient->ViewFOV = 90;
					}
					else
					{
						for (TActorIterator<APlayerStart> It(World); It; ++It)
						{
							if (APlayerStart* PlayerStart = *It; IsValid(PlayerStart))
							{
								ViewportClient->SetViewLocation(PlayerStart->GetActorLocation());
								ViewportClient->SetViewRotation(PlayerStart->GetActorRotation());
								ViewportClient->ViewFOV = 90;
								break;
							}
						}
					}
				}

				ViewportClient->Viewport->Draw();
				Grid->EndViewportRender(RenderGridJob);

				BlueprintEditor->SetIsDebugging(false);
			}
		}
	}
}

void UE::RenderGrid::Private::SRenderGridEditorViewport::ClearSequenceFrame()
{
	ShowSequenceFrame(nullptr, nullptr, 0);
}

bool UE::RenderGrid::Private::SRenderGridEditorViewport::ShowSequenceFrame(URenderGridJob* InJob, ULevelSequence* InSequence, const float InTime)
{
	RenderGridJob = InJob;
	LevelSequenceTime = InTime;
	if (!IsValid(InSequence))
	{
		LevelSequence = nullptr;
		DestroySequencePlayer();
		return false;
	}
	if (!IsValid(LevelSequence) || (LevelSequence != InSequence))
	{
		LevelSequence = InSequence;
		DestroySequencePlayer();
	}
	return true;
}

ULevelSequencePlayer* UE::RenderGrid::Private::SRenderGridEditorViewport::GetSequencePlayer()
{
	if (!IsValid(LevelSequence))
	{
		return nullptr;
	}
	if (UWorld* World = GetWorld(); IsValid(World))
	{
		if (IsValid(LevelSequencePlayer) && LevelSequencePlayerWorld.IsValid() && (World == LevelSequencePlayerWorld))
		{
			return LevelSequencePlayer;
		}

		LevelSequencePlayerWorld = nullptr;
		LevelSequencePlayerActor = nullptr;
		LevelSequencePlayer = nullptr;

		FMovieSceneSequencePlaybackSettings PlaybackSettings;
		PlaybackSettings.LoopCount.Value = 0;
		PlaybackSettings.bAutoPlay = false;
		PlaybackSettings.bPauseAtEnd = true;
		PlaybackSettings.FinishCompletionStateOverride = EMovieSceneCompletionModeOverride::ForceRestoreState;

		ALevelSequenceActor* PlayerActor = nullptr;
		if (ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, PlaybackSettings, PlayerActor); IsValid(Player))
		{
			if (IsValid(PlayerActor))
			{
				LevelSequencePlayerWorld = World;
				LevelSequencePlayerActor = PlayerActor;
				LevelSequencePlayer = Player;
				return Player;
			}
		}
	}
	return nullptr;
}

void UE::RenderGrid::Private::SRenderGridEditorViewport::DestroySequencePlayer()
{
	// TObjectPtr<ALevelSequenceActor> PlayerActor = LevelSequencePlayerActor;
	// TObjectPtr<ULevelSequencePlayer> Player = LevelSequencePlayer;

	LevelSequencePlayerWorld = nullptr;
	LevelSequencePlayerActor = nullptr;
	LevelSequencePlayer = nullptr;

	//  Can cause crashes (when opening a different Map in the editor):
	// if (IsValid(Player))
	// {
	//     Player->Stop();
	// }
	// if (IsValid(PlayerActor))
	// {
	//     PlayerActor->Destroy(false, false);
	// }
}


void UE::RenderGrid::Private::SRenderGridViewerLive::Tick(const FGeometry&, const double, const float)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		UpdateActionButton();
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewerLive::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	SelectedJobWeakPtr = nullptr;

	SAssignNew(ViewportWidget, SRenderGridEditorViewport, InBlueprintEditor);

	SAssignNew(FrameSlider, SRenderGridViewerFrameSlider)
		.Visibility(EVisibility::Hidden)
		.OnValueChanged(this, &SRenderGridViewerLive::FrameSliderValueChanged);

	SelectedJobChanged();
	ViewportWidget->Render(); // prevents the waiting text from showing up for 1 frame when switching from any other viewer mode to the live viewer mode

	InBlueprintEditor->OnRenderGridChanged().AddSP(this, &SRenderGridViewerLive::RenderGridJobDataChanged);
	InBlueprintEditor->OnRenderGridJobsSelectionChanged().AddSP(this, &SRenderGridViewerLive::SelectedJobChanged);
	FCoreUObjectDelegates::OnObjectModified.AddSP(this, &SRenderGridViewerLive::OnObjectModified);

	ChildSlot
	[
		SNew(SVerticalBox)

		// viewport & waiting text
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f)
		[
			SNew(SOverlay)

			// black background
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Brushes.White"))
				.ColorAndOpacity(FLinearColor(0, 0, 0, 1))
			]

			// viewport
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SScaleBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Stretch(EStretch::ScaleToFit)
				.StretchDirection(EStretchDirection::Both)
				[
					SNew(SBox)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.WidthOverride(1.0f)
					.HeightOverride_Lambda([this]() -> float { return 1.0 / (SelectedJobWeakPtr.IsValid() ? SelectedJobWeakPtr->GetOutputAspectRatio() : 1.0); })
					[
						ViewportWidget.ToSharedRef()
					]
				]
			]

			// waiting text & background
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SOverlay)
				.Visibility_Lambda([this]() -> EVisibility { return (ViewportWidget->HasRenderedLastAttempt() ? EVisibility::Hidden : EVisibility::Visible); })

				// waiting text background (hides the viewport)
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Brushes.White"))
					.ColorAndOpacity(FLinearColor(0, 0, 0, 1))
				]

				// waiting text button
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ActionButton, SButton)
					.ContentPadding(0.0f)
					.ButtonStyle(FRenderGridEditorStyle::Get(), TEXT("Invisible"))
					.IsFocusable(false)
					.OnClicked(this, &SRenderGridViewerLive::OnClicked)
					[
						SNew(SOverlay)

						// waiting text
						+ SOverlay::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text_Lambda([this]() -> FText
							{
								if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
								{
									if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
									{
										if (Grid->GetLevel().IsNull())
										{
											return LOCTEXT("PleaseSelectLevelForGrid", "Please select a level for this grid");
										}
										if (!Grid->GetLevel().IsValid())
										{
											return LOCTEXT("ClickToOpenLevelOfGrid", "Click here to open the level of this grid");
										}
										if (Grid->GetLevel().Get() != ViewportWidget->GetWorldPublic())
										{
											return LOCTEXT("ViewerWrongWorldPleaseTryReopening", "The viewer is currently trying to view the incorrect world, please try reopening the RenderGrid");
										}
										if (!Grid->HasAnyRenderGridJobs())
										{
											return LOCTEXT("PleaseAddJob", "Please add a job");
										}
									}
								}
								if (URenderGridJob* SelectedJob = SelectedJobWeakPtr.Get(); IsValid(SelectedJob))
								{
									if (ULevelSequence* Sequence = SelectedJob->GetLevelSequence(); IsValid(Sequence))
									{
										return LOCTEXT("WaitingForRenderer", "Waiting for renderer...");
									}
									return LOCTEXT("PleaseSelectLevelSequenceForJob", "Please select a level sequence for this job");
								}
								return LOCTEXT("PleaseSelectJob", "Please select a job");
							})
						]
					]
				]
			]
		]

		// slider
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f)
		[
			FrameSlider.ToSharedRef()
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply UE::RenderGrid::Private::SRenderGridViewerLive::OnClicked()
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (!Grid->GetLevel().IsNull() && !Grid->GetLevel().IsValid())
			{
				FEditorFileUtils::LoadMap(Grid->GetLevel().ToString(), false, true);
				return FReply::Handled();
			}
		}
	}
	return FReply::Handled();
}

void UE::RenderGrid::Private::SRenderGridViewerLive::OnObjectModified(UObject* Object)
{
	if (SelectedJobWeakPtr.IsValid() && (Object == SelectedJobWeakPtr))
	{
		// selected render grid job changed
		SelectedJobChanged();
	}
	else if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (Object == BlueprintEditor->GetInstance())
		{
			// render grid changed
			RenderGridJobDataChanged();
		}
	}
}

void UE::RenderGrid::Private::SRenderGridViewerLive::RenderGridJobDataChanged()
{
	UpdateViewport();
	UpdateFrameSlider();
}

void UE::RenderGrid::Private::SRenderGridViewerLive::SelectedJobChanged()
{
	SelectedJobWeakPtr = nullptr;
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (const TArray<URenderGridJob*> SelectedJobs = BlueprintEditor->GetSelectedRenderGridJobs(); (SelectedJobs.Num() == 1))
		{
			SelectedJobWeakPtr = SelectedJobs[0];
		}
	}

	UpdateViewport();
	UpdateFrameSlider();
}

void UE::RenderGrid::Private::SRenderGridViewerLive::FrameSliderValueChanged(const float NewValue)
{
	UpdateViewport();
	UpdateFrameSlider();
}


void UE::RenderGrid::Private::SRenderGridViewerLive::UpdateViewport()
{
	if (!ViewportWidget.IsValid() || !FrameSlider.IsValid())
	{
		return;
	}

	if (URenderGridJob* SelectedJob = SelectedJobWeakPtr.Get(); IsValid(SelectedJob))
	{
		if (ULevelSequence* Sequence = SelectedJob->GetLevelSequence(); IsValid(Sequence))
		{
			if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
			{
				TOptional<double> StartTime = SelectedJob->GetStartTime();
				TOptional<double> EndTime = SelectedJob->GetEndTime();
				if (StartTime.IsSet() && EndTime.IsSet() && (StartTime.Get(0) <= EndTime.Get(0)))
				{
					ViewportWidget->ShowSequenceFrame(SelectedJob, Sequence, FMath::Lerp(StartTime.Get(0), EndTime.Get(0), FrameSlider->GetValue()));
					return;
				}
			}
		}
	}
	ViewportWidget->ClearSequenceFrame();
}

void UE::RenderGrid::Private::SRenderGridViewerLive::UpdateActionButton()
{
	if (!ActionButton.IsValid())
	{
		return;
	}

	bool bIsUsable = false;

	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (!Grid->GetLevel().IsNull() && !Grid->GetLevel().IsValid())
			{
				bIsUsable = true;
			}
		}
	}

	ActionButton->SetButtonStyle(&FRenderGridEditorStyle::Get().GetWidgetStyle<FButtonStyle>(bIsUsable ? TEXT("HoverHintOnly") : TEXT("Invisible")));
	ActionButton->SetCursor(bIsUsable ? EMouseCursor::Type::Hand : EMouseCursor::Type::Default);
}

void UE::RenderGrid::Private::SRenderGridViewerLive::UpdateFrameSlider()
{
	if (!FrameSlider.IsValid())
	{
		return;
	}
	FrameSlider->SetVisibility(EVisibility::Hidden);

	if (URenderGridJob* SelectedJob = SelectedJobWeakPtr.Get(); IsValid(SelectedJob))
	{
		if (!FrameSlider->SetFramesText(SelectedJob))
		{
			return;
		}

		FrameSlider->SetVisibility(EVisibility::Visible);
	}
}


#undef LOCTEXT_NAMESPACE
