// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelSequence.h"
#include "LevelSequencePlayer.h"
#include "Widgets/SCompoundWidget.h"
#include "SEditorViewport.h"


class ALevelSequenceActor;
class SButton;
class SSlider;
class URenderGridJob;
class URenderGrid;

namespace UE::RenderGrid
{
	class IRenderGridEditor;
}

namespace UE::RenderGrid::Private
{
	class SRenderGridViewerFrameSlider;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The viewport client for the live render grid viewer widget.
	 */
	class FRenderGridEditorViewportClient : public FEditorViewportClient
	{
	public:
		explicit FRenderGridEditorViewportClient(FPreviewScene* PreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr);

	public:
		//~ Begin FEditorViewportClient Interface
		virtual EMouseCursor::Type GetCursor(FViewport* InViewport, int32 X, int32 Y) override { return EMouseCursor::Default; }
		//~ End FEditorViewportClient Interface
	};


	/**
	 * The viewport for the live render grid viewer widget.
	 */
	class SRenderGridEditorViewport : public SEditorViewport
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridEditorViewport) {}
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);
		virtual ~SRenderGridEditorViewport() override;

		void Render();
		void ClearSequenceFrame();
		bool ShowSequenceFrame(URenderGridJob* InJob, ULevelSequence* InSequence, const float InTime);
		bool HasRenderedLastAttempt() const { return bRenderedLastAttempt; }
		UWorld* GetWorldPublic() const { return GetWorld(); }

	protected:
		//~ Begin SEditorViewport Interface
		virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override { return ViewportClient.ToSharedRef(); }
		virtual void BindCommands() override {}
		virtual bool SupportsKeyboardFocus() const override { return false; }
		//~ End SEditorViewport Interface

		ULevelSequencePlayer* GetSequencePlayer();
		void DestroySequencePlayer();

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** The viewport client. */
		TSharedPtr<FRenderGridEditorViewportClient> ViewportClient;

		/** The world that the level sequence actor was spawned in. */
		UPROPERTY()
		TWeakObjectPtr<UWorld> LevelSequencePlayerWorld;

		/** The level sequence actor we spawned to play the sequence of any given render grid job. */
		UPROPERTY()
		TObjectPtr<ALevelSequenceActor> LevelSequencePlayerActor;

		/** The level sequence player we created to play the sequence of any given render grid job. */
		UPROPERTY()
		TObjectPtr<ULevelSequencePlayer> LevelSequencePlayer;

		/** The level sequence that we're currently playing. */
		UPROPERTY()
		TObjectPtr<ULevelSequence> LevelSequence;

		/** The render grid job that's currently being shown. */
		UPROPERTY()
		TObjectPtr<URenderGridJob> RenderGridJob;

		/** The time of the currently playing sequence. */
		UPROPERTY()
		float LevelSequenceTime;

		/** Whether it rendered or not during the last tick. */
		UPROPERTY()
		bool bRenderedLastAttempt;
	};


	/**
	 * A render grid viewer widget, allows the user to see a live render using a viewport.
	 */
	class SRenderGridViewerLive : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridViewerLive) {}
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

	private:
		FReply OnClicked();
		void OnObjectModified(UObject* Object);
		void RenderGridJobDataChanged();
		void SelectedJobChanged();
		void FrameSliderValueChanged(const float NewValue);

		void UpdateViewport();
		void UpdateActionButton();
		void UpdateFrameSlider();

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** A reference to the job that's currently rendering. */
		TWeakObjectPtr<URenderGridJob> SelectedJobWeakPtr;

		/** The viewport widget. */
		TSharedPtr<SRenderGridEditorViewport> ViewportWidget;

		/** The widget that allows the user to select what frame they'd like to see. */
		TSharedPtr<SRenderGridViewerFrameSlider> FrameSlider;

		/** The button which can be clicked, to to rerender the preview, or to open the map of the grid. */
		TSharedPtr<SButton> ActionButton;
	};
}
