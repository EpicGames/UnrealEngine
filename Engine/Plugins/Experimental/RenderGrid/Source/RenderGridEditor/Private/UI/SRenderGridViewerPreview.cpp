// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRenderGridViewerPreview.h"
#include "UI/Components/SRenderGridViewerFrameSlider.h"
#include "RenderGrid/RenderGrid.h"
#include "RenderGrid/RenderGridQueue.h"
#include "IRenderGridEditor.h"
#include "IRenderGridModule.h"
#include "LevelSequence.h"
#include "MoviePipelineOutputSetting.h"
#include "MovieScene.h"
#include "RenderGrid/RenderGridManager.h"
#include "SlateOptMacros.h"
#include "TextureResource.h"
#include "Engine/Texture2D.h"
#include "Sections/MovieSceneSubSection.h"
#include "Styles/RenderGridEditorStyle.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScaleBox.h"
#include "FileHelpers.h"

#define LOCTEXT_NAMESPACE "SRenderGridViewerPreview"


bool UE::RenderGrid::Private::SRenderGridViewerPreview::bHasRenderedSinceAppStart = false;


void UE::RenderGrid::Private::SRenderGridViewerPreview::Tick(const FGeometry&, const double, const float)
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		UpdateActionButton();

		if (BlueprintEditor->CanCurrentlyRender() && !CurrentRenderQueueWeakPtr.IsValid())
		{
			UpdateFrameSlider();

			if (FramesUntilRenderNewPreview > 0)
			{
				FramesUntilRenderNewPreview--;
				if (FramesUntilRenderNewPreview <= 0)
				{
					InternalRenderNewPreview();
				}
			}
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewerPreview::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	BlueprintEditorWeakPtr = InBlueprintEditor;
	SelectedJobWeakPtr = nullptr;
	CurrentRenderQueueWeakPtr = nullptr;
	FramesUntilRenderNewPreview = 0;
	ImageBrushEmpty = FSlateBrush();
	ImageBrushEmpty.DrawAs = ESlateBrushDrawType::Type::NoDrawType;
	ImageBrush = FSlateBrush();
	ImageTexture = nullptr;
	LastUpdateImageTextureSelectedJobWeakPtr = nullptr;

	SAssignNew(Image, SImage)
		.Image(&ImageBrushEmpty);

	SAssignNew(ImageBackground, SImage)
		.Image(&ImageBrushEmpty);

	SAssignNew(FrameSlider, SRenderGridViewerFrameSlider)
		.Visibility(EVisibility::Hidden)
		.OnValueChanged(this, &SRenderGridViewerPreview::FrameSliderValueChanged)
		.OnCaptureEnd(this, &SRenderGridViewerPreview::FrameSliderValueChangedEnd);

	SelectedJobChanged();

	InBlueprintEditor->OnRenderGridChanged().AddSP(this, &SRenderGridViewerPreview::GridDataChanged);
	InBlueprintEditor->OnRenderGridJobsSelectionChanged().AddSP(this, &SRenderGridViewerPreview::SelectedJobChanged);
	FCoreUObjectDelegates::OnObjectModified.AddSP(this, &SRenderGridViewerPreview::OnObjectModified);

	ChildSlot
	[
		SNew(SVerticalBox)
		.Visibility_Lambda([this]() -> EVisibility { return (!IsPreviewWidget() && CurrentRenderQueueWeakPtr.IsValid()) ? EVisibility::Hidden : EVisibility::Visible; })

		// image
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f)
		[
			SNew(SOverlay)

			// black background
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Brushes.White"))
				.ColorAndOpacity(FLinearColor(0, 0, 0, 1))
			]

			// image button
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(ActionButton, SButton)
				.ContentPadding(0.0f)
				.ButtonStyle(FRenderGridEditorStyle::Get(), TEXT("Invisible"))
				.IsFocusable(false)
				.OnClicked(this, &SRenderGridViewerPreview::OnClicked)
				[
					SNew(SOverlay)

					// waiting text
					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text_Lambda([this]() -> FText
						{
							if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
							{
								if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
								{
									if (Grid->GetLevel().IsNull())
									{
										return LOCTEXT("PleaseSelectLevelForGrid", "Please select a level for this grid");
									}
									if (!Grid->GetLevel().IsValid())
									{
										return LOCTEXT("ClickToOpenLevelOfGrid", "Click here to open the level of this grid");
									}
									if (!Grid->HasAnyRenderGridJobs())
									{
										return LOCTEXT("PleaseAddJob", "Please add a job");
									}
								}
							}
							if (URenderGridJob* SelectedJob = SelectedJobWeakPtr.Get(); IsValid(SelectedJob))
							{
								if (ULevelSequence* Sequence = SelectedJob->GetLevelSequence(); IsValid(Sequence))
								{
									if (IsPreviewWidget() || !FrameSlider.IsValid() || (FrameSlider->GetVisibility() != EVisibility::Visible))
									{
										return LOCTEXT("WaitingForRenderer", "Waiting for renderer...");
									}
									return LOCTEXT("ClickToStartRendering", "Click here to start rendering");
								}
								return LOCTEXT("PleaseSelectLevelSequenceForJob", "Please select a level sequence for this job");
							}
							return LOCTEXT("PleaseSelectJob", "Please select a job");
						})
					]

					// image & background
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SScaleBox)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Stretch(EStretch::ScaleToFit)
						.StretchDirection(EStretchDirection::Both)
						[
							SNew(SOverlay)

							// background
							+ SOverlay::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								ImageBackground.ToSharedRef()
							]

							// image
							+ SOverlay::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								Image.ToSharedRef()
							]
						]
					]
				]
			]
		]

		// slider
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f)
		[
			FrameSlider.ToSharedRef()
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


FReply UE::RenderGrid::Private::SRenderGridViewerPreview::OnClicked()
{
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (!Grid->GetLevel().IsNull() && !Grid->GetLevel().IsValid())
			{
				FEditorFileUtils::LoadMap(Grid->GetLevel().ToString(), false, true);
				return FReply::Handled();
			}
		}
	}

	if (!IsPreviewWidget())
	{
		RenderNewPreview();
	}
	return FReply::Handled();
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::OnObjectModified(UObject* Object)
{
	if (SelectedJobWeakPtr.IsValid() && (Object == SelectedJobWeakPtr.Get()))
	{
		// selected job changed
		SelectedJobChanged();
	}
	else if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (Object == BlueprintEditor->GetInstance())
		{
			// grid changed
			GridDataChanged();
		}
	}
	else if (UMoviePipelineOutputSetting* Settings = Cast<UMoviePipelineOutputSetting>(Object))
	{
		if (SelectedJobWeakPtr.IsValid() && (Settings == SelectedJobWeakPtr->GetRenderPresetOutputSettings()))
		{
			// movie pipeline output settings changed
			GridDataChanged();
		}
	}
	else if (SelectedJobWeakPtr.IsValid() && SelectedJobWeakPtr->GetLevelSequence() && (Cast<UMovieSceneSequence>(Object) || Cast<UMovieScene>(Object) || Cast<UMovieSceneTrack>(Object) || Cast<UMovieSceneSection>(Object) || Cast<UMovieSceneSubTrack>(Object) || Cast<UMovieSceneSubSection>(Object)))
	{
		ULevelSequence* Sequence = SelectedJobWeakPtr->GetLevelSequence();
		if ((Object == Sequence) || (Object == Sequence->GetMovieScene()) || (Object->GetTypedOuter<UMovieScene>() == Sequence->GetMovieScene()))
		{
			// level sequence changed
			GridDataChanged();
		}
	}
}


void UE::RenderGrid::Private::SRenderGridViewerPreview::GridDataChanged()
{
	if (IsPreviewWidget())
	{
		UpdateImageTexture();
		RenderNewPreview();
	}
	else
	{
		UpdateImageTexture();
	}
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::SelectedJobChanged()
{
	SelectedJobWeakPtr = nullptr;
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (const TArray<URenderGridJob*> SelectedJobs = BlueprintEditor->GetSelectedRenderGridJobs(); (SelectedJobs.Num() == 1))
		{
			SelectedJobWeakPtr = SelectedJobs[0];
		}
	}

	if (IsPreviewWidget())
	{
		UpdateImageTexture();
		RenderNewPreview();
	}
	else
	{
		UpdateImageTexture();
	}
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::FrameSliderValueChanged(const float NewValue)
{
	if (!IsPreviewWidget())
	{
		UpdateImageTexture(false);
	}
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::FrameSliderValueChangedEnd()
{
	if (IsPreviewWidget())
	{
		RenderNewPreview();
	}
	else
	{
		UpdateImageTexture();
	}
}


void UE::RenderGrid::Private::SRenderGridViewerPreview::RenderNewPreview()
{
	FramesUntilRenderNewPreview = 1;
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::InternalRenderNewPreview()
{
	InternalRenderNewPreviewOfRenderGridJob(SelectedJobWeakPtr.Get());
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::InternalRenderNewPreviewOfRenderGridJob(URenderGridJob* Job)
{
	if ((GetTickSpaceGeometry().Size.X <= 0) || (IsPreviewWidget() && (GetColorPicker() != nullptr)))
	{
		// don't render, try again next frame
		RenderNewPreview();
		return;
	}

	if (!IsValid(Job))
	{
		SetImageTexture(nullptr);
		return;
	}

	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (Grid->GetLevel().IsNull() || !Grid->GetLevel().IsValid())
			{
				// don't render, clear image and try again next frame
				SetImageTexture(nullptr);
				RenderNewPreview();
				return;
			}
		}

		double WidgetWidth = FMath::Max(120.0, FMath::Min(GetTickSpaceGeometry().Size.X, GetTickSpaceGeometry().Size.Y * Job->GetOutputAspectRatio()));
		double RenderResolution = WidgetWidth * (IsPreviewWidget() ? 1.25 : 0.75); // pixels in width

		FRenderGridManagerRenderPreviewFrameArgs JobArgs;
		if (IsPreviewWidget())
		{
			TOptional<int32> SelectedFrame = (!FrameSlider.IsValid() ? TOptional<int32>() : FrameSlider->GetSelectedFrame(Job));
			if (!SelectedFrame.IsSet())
			{
				SetImageTexture(nullptr);
				return;
			}

			JobArgs.bHeadless = bHasRenderedSinceAppStart;
			JobArgs.Frame = SelectedFrame.Get(0);
		}
		JobArgs.RenderGrid = TStrongObjectPtr(BlueprintEditor->GetInstance());
		JobArgs.RenderGridJob = TStrongObjectPtr(Job);
		JobArgs.Resolution = FIntPoint(FMath::RoundToInt32(RenderResolution), FMath::RoundToInt32(RenderResolution / Job->GetOutputAspectRatio()));

		const TSharedPtr<SWidget> BaseThis = AsShared();
		const TSharedPtr<SRenderGridViewerPreview> This = StaticCastSharedPtr<SRenderGridViewerPreview>(BaseThis);
		JobArgs.Callback = FRenderGridManagerRenderPreviewFrameArgsCallback::CreateLambda([This, BlueprintEditor](const bool bSuccess)
		{
			if (This.IsValid())
			{
				This->RenderNewPreviewCallback(bSuccess);
			}
			else if (BlueprintEditor.IsValid())
			{
				BlueprintEditor->SetPreviewRenderQueue(nullptr);
			}
		});

		if (URenderGridQueue* NewRenderQueue = IRenderGridModule::Get().GetManager().RenderPreviewFrame(JobArgs))
		{
			CurrentRenderQueueWeakPtr = NewRenderQueue;
			BlueprintEditor->SetPreviewRenderQueue(NewRenderQueue);
			return;
		}
	}
	SetImageTexture(nullptr);
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::RenderNewPreviewCallback(const bool bSuccess)
{
	UpdateImageTexture();

	CurrentRenderQueueWeakPtr = nullptr;
	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		BlueprintEditor->SetPreviewRenderQueue(nullptr);
		if (!bHasRenderedSinceAppStart)
		{
			BlueprintEditor->OnRenderGridPreviewRenderedFirstTimeSinceAppStart().Broadcast();
		}
	}
	bHasRenderedSinceAppStart = true;
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::UpdateImageTexture(const bool bForce)
{
	if (!FrameSlider.IsValid())
	{
		SetImageTexture(nullptr);
		LastUpdateImageTextureSelectedJobWeakPtr = nullptr;
		LastUpdateImageTextureFrame = TOptional<int32>();
		return;
	}

	if (URenderGridJob* SelectedJob = SelectedJobWeakPtr.Get(); IsValid(SelectedJob))
	{
		if (!bForce && LastUpdateImageTextureSelectedJobWeakPtr.IsValid() && (LastUpdateImageTextureSelectedJobWeakPtr == SelectedJob))
		{
			if (IsPreviewWidget() || (LastUpdateImageTextureFrame == FrameSlider->GetSelectedFrame(SelectedJob)))
			{
				return; // nothing changed
			}
		}
		LastUpdateImageTextureSelectedJobWeakPtr = SelectedJob;

		if (IsPreviewWidget())
		{
			if (!FrameSlider->GetSelectedFrame(SelectedJob).IsSet())
			{
				SetImageTexture(nullptr);
				return;
			}
			SetImageTexture(IRenderGridModule::Get().GetManager().GetSingleRenderedPreviewFrame(SelectedJob));
			return;
		}

		TOptional<int32> Frame = FrameSlider->GetSelectedFrame(SelectedJob);
		if (!Frame.IsSet())
		{
			LastUpdateImageTextureFrame = Frame;
			SetImageTexture(nullptr);
			return;
		}

		if (UTexture2D* Texture = IRenderGridModule::Get().GetManager().GetRenderedPreviewFrame(SelectedJob, Frame.Get(0)); IsValid(Texture))
		{
			LastUpdateImageTextureFrame = Frame;
			SetImageTexture(Texture);
			return;
		}

		LastUpdateImageTextureFrame = TOptional<int32>();
		SetImageTexture(nullptr);
	}
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::SetImageTexture(UTexture2D* Texture)
{
	{// cleanup >>
		if (Image.IsValid())
		{
			Image->SetImage(&ImageBrushEmpty);
			ImageBackground->SetImage(&ImageBrushEmpty);
		}
		ImageBrush.SetResourceObject(nullptr);
		ImageBrush.SetImageSize(FVector2D(0, 0));
		ImageTexture = nullptr;
	}// cleanup <<

	{// set new texture >>
		if (IsValid(Texture) && Texture->GetResource())
		{
			ImageTexture = TStrongObjectPtr(Texture);
			if (Image.IsValid())
			{
				static constexpr double PreviewTabAspectRatio = 1.96875;

				double ImageWidth = ImageTexture->GetResource()->GetSizeX();
				double ImageHeight = ImageTexture->GetResource()->GetSizeY();
				const double ImageAspectRatio = ImageWidth / ImageHeight;
				if (ImageAspectRatio > PreviewTabAspectRatio)
				{
					ImageWidth = 1280.0;
					ImageHeight = ImageWidth / ImageAspectRatio;
				}
				else
				{
					ImageHeight = 1280.0 / PreviewTabAspectRatio;
					ImageWidth = ImageHeight * ImageAspectRatio;
				}

				ImageBrush = FSlateBrush();
				ImageBrush.DrawAs = ESlateBrushDrawType::Type::Image;
				ImageBrush.ImageType = ESlateBrushImageType::Type::FullColor;
				ImageBrush.SetResourceObject(ImageTexture.Get());
				ImageBrush.SetImageSize(FVector2D(ImageWidth, ImageHeight));
				Image->SetImage(&ImageBrush);
				ImageBackground->SetImage(FCoreStyle::Get().GetBrush("Checkerboard"));
			}
		}
	}// set new texture <<
}


void UE::RenderGrid::Private::SRenderGridViewerPreview::UpdateActionButton()
{
	if (!ActionButton.IsValid())
	{
		return;
	}

	bool bIsUsable = (!IsPreviewWidget() && SelectedJobWeakPtr.IsValid());

	if (const TSharedPtr<IRenderGridEditor> BlueprintEditor = BlueprintEditorWeakPtr.Pin())
	{
		if (URenderGrid* Grid = BlueprintEditor->GetInstance(); IsValid(Grid))
		{
			if (!Grid->GetLevel().IsNull() && !Grid->GetLevel().IsValid())
			{
				bIsUsable = true;
			}
		}
	}

	ActionButton->SetButtonStyle(&FRenderGridEditorStyle::Get().GetWidgetStyle<FButtonStyle>(bIsUsable ? TEXT("HoverHintOnly") : TEXT("Invisible")));
	ActionButton->SetCursor(bIsUsable ? EMouseCursor::Type::Hand : EMouseCursor::Type::Default);
}

void UE::RenderGrid::Private::SRenderGridViewerPreview::UpdateFrameSlider()
{
	if (!FrameSlider.IsValid())
	{
		return;
	}
	EVisibility LastVisibility = FrameSlider->GetVisibility();
	FrameSlider->SetVisibility(EVisibility::Hidden);

	if (URenderGridJob* SelectedJob = SelectedJobWeakPtr.Get(); IsValid(SelectedJob))
	{
		if (IsPreviewWidget() && !FrameSlider->GetSelectedSequenceFrame(SelectedJob).IsSet())
		{
			return;
		}

		if (!FrameSlider->SetFramesText(SelectedJob))
		{
			return;
		}

		FrameSlider->SetVisibility(EVisibility::Visible);
		if (!IsPreviewWidget() && (LastVisibility != EVisibility::Visible))
		{
			UpdateImageTexture();
		}
	}
}


#undef LOCTEXT_NAMESPACE
