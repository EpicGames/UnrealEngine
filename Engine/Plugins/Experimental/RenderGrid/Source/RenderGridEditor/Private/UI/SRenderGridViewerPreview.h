// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateBrush.h"


class SButton;
class SImage;
class SSlider;
class URenderGridJob;
class URenderGridQueue;


namespace UE::RenderGrid
{
	class IRenderGridEditor;
}

namespace UE::RenderGrid::Private
{
	class SRenderGridViewerFrameSlider;
	struct FRenderGridManagerRenderPreviewFrameResult;
}


namespace UE::RenderGrid::Private
{
	/**
	 * A render grid viewer widget, allows the user to render a single frame of a render grid job in low-resolution and afterwards see it in the editor.
	 */
	class SRenderGridViewerPreview : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridViewerPreview) {}
		SLATE_END_ARGS()

		virtual void Tick(const FGeometry&, const double, const float) override;
		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);

		/** Queues a new render action (if it isn't yet queued already). This is basically the Refresh() function of this widget. */
		void RenderNewPreview();

		/** Retrieves the rendered output from disk and shows it in the UI. */
		void UpdateImageTexture(const bool bForce = true);

		/** Shows the given texture, this is normally the output of a render. */
		void SetImageTexture(UTexture2D* Texture);

	protected:
		/** Returns true if this is the preview widget (1 frame), returns false if this is the rendered widgets (all frames). Override this function in order to change the value it returns. */
		virtual bool IsPreviewWidget() const { return true; }

	private:
		FReply OnClicked();
		void OnObjectModified(UObject* Object);
		void GridDataChanged();
		void SelectedJobChanged();
		void FrameSliderValueChanged(const float NewValue);
		void FrameSliderValueChangedEnd();

		void InternalRenderNewPreview();
		void InternalRenderNewPreviewOfRenderGridJob(URenderGridJob* Job);
		void RenderNewPreviewCallback(const bool bSuccess);

		void UpdateActionButton();
		void UpdateFrameSlider();

	private:
		/** A reference to the blueprint editor that owns the render grid instance. */
		TWeakPtr<IRenderGridEditor> BlueprintEditorWeakPtr;

		/** A reference to the queue that's currently rendering. */
		TWeakObjectPtr<URenderGridQueue> CurrentRenderQueueWeakPtr;

		/** A reference to the job that's currently rendering. */
		TWeakObjectPtr<URenderGridJob> SelectedJobWeakPtr;

		/** Has a value of 1 if it should call RenderNewPreview() next frame, 2+ if it should subtract 1 from its value next frame, 0 and below and it won't do anything. */
		int32 FramesUntilRenderNewPreview;

	private:
		/** True if it has rendered before since the start of this application. Used for not hiding the rendering popup during the first render (since the first render can take a lot longer due to having to compile shaders etc). */
		static bool bHasRenderedSinceAppStart;

		/** The widget that allows the user to select what frame they'd like to see. */
		TSharedPtr<SRenderGridViewerFrameSlider> FrameSlider;

		/** The button which can be clicked, to to rerender the preview, or to open the map of the grid. */
		TSharedPtr<SButton> ActionButton;

		/** The widget that contains the image. */
		TSharedPtr<SImage> Image;

		/** The widget that contains the background of the image. */
		TSharedPtr<SImage> ImageBackground;

		/** The brush of the image, always empty. */
		FSlateBrush ImageBrushEmpty;

		/** The brush of the image. */
		FSlateBrush ImageBrush;

		/** The texture of the image. */
		TStrongObjectPtr<UTexture2D> ImageTexture;

		/** The last render grid job used for the LastUpdateImageTexture function. */
		TWeakObjectPtr<URenderGridJob> LastUpdateImageTextureSelectedJobWeakPtr;

		/** The last frame used for the LastUpdateImageTexture function. */
		TOptional<int32> LastUpdateImageTextureFrame;
	};
}
