// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/SRenderGridViewerPreview.h"


namespace UE::RenderGrid::Private
{
	/**
	 * A render grid viewer widget, allows the user to render a render grid job in low-resolution and afterwards scrub through the outputted frames of it in the editor.
	 */
	class SRenderGridViewerRendered : public SRenderGridViewerPreview
	{
	protected:
		virtual bool IsPreviewWidget() const override { return false; }
	};
}
