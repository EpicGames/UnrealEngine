// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Tabs/SRenderGridJobListTab.h"
#include "UI/SRenderGridJobList.h"
#include "IRenderGridEditor.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SRenderGridJobListTab"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridJobListTab::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	ChildSlot
	[
		SNew(SRenderGridJobList, InBlueprintEditor)
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
