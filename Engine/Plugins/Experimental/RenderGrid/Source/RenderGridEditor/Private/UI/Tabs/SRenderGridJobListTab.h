// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


namespace UE::RenderGrid
{
	class IRenderGridEditor;
}


namespace UE::RenderGrid::Private
{
	/**
	 * The render grid job list tab.
	 */
	class SRenderGridJobListTab : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRenderGridJobListTab) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor);
	};
}
