// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Tabs/SRenderGridJobPropertiesTab.h"
#include "UI/SRenderGridJob.h"
#include "UI/SRenderGridProps.h"
#include "IRenderGridEditor.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SRenderGridJobPropertiesTab"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridJobPropertiesTab::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	ChildSlot
	[
		SNew(SScrollBox)
		.Style(FAppStyle::Get(), "ScrollBox")

		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f)
			[
				SNew(SRenderGridJob, InBlueprintEditor)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f)
			[
				SNew(SRenderGridProps, InBlueprintEditor)
			]
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
