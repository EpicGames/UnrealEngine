// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Tabs/SRenderGridPropertiesTab.h"
#include "UI/SRenderGrid.h"
#include "IRenderGridEditor.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SRenderGridPropertiesTab"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridPropertiesTab::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	ChildSlot
	[
		SNew(SScrollBox)
		.Style(FAppStyle::Get(), "ScrollBox")

		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f)
			[
				SNew(SRenderGrid, InBlueprintEditor)
			]
		]
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
