// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Tabs/SRenderGridViewerTab.h"
#include "UI/SRenderGridViewer.h"
#include "IRenderGridEditor.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "SRenderGridViewerTab"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void UE::RenderGrid::Private::SRenderGridViewerTab::Construct(const FArguments& InArgs, TSharedPtr<IRenderGridEditor> InBlueprintEditor)
{
	ChildSlot
	[
		SNew(SRenderGridViewer, InBlueprintEditor)
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
