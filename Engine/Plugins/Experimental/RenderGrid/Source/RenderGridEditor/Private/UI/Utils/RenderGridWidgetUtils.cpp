// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Utils/RenderGridWidgetUtils.h"
#include "IDetailTreeNode.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "RenderGridWidgetUtils"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> UE::RenderGrid::Private::RenderGridWidgetUtils::CreateNodeValueWidget(const TSharedPtr<IDetailTreeNode>& Node)
{
	FNodeWidgets NodeWidgets = Node->CreateNodeWidgets();

	TSharedRef<SHorizontalBox> FieldWidget = SNew(SHorizontalBox);

	if (NodeWidgets.ValueWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.HAlign(HAlign_Right)
			.FillWidth(1.0f)
			[
				NodeWidgets.ValueWidget.ToSharedRef()
			];
	}
	else if (NodeWidgets.WholeRowWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.FillWidth(1.0f)
			[
				NodeWidgets.WholeRowWidget.ToSharedRef()
			];
	}

	return FieldWidget;
}

bool UE::RenderGrid::Private::RenderGridWidgetUtils::FindPropertyHandleRecursive(const TSharedPtr<IPropertyHandle>& PropertyHandle, const FString& PropertyNameOrPath, ERenderGridFindNodeMethod FindMethod)
{
	if (PropertyHandle && PropertyHandle->IsValidHandle())
	{
		uint32 ChildrenCount = 0;
		PropertyHandle->GetNumChildren(ChildrenCount);
		for (uint32 Index = 0; Index < ChildrenCount; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(Index);
			if (FindPropertyHandleRecursive(ChildHandle, PropertyNameOrPath, FindMethod))
			{
				return true;
			}
		}

		if (PropertyHandle->GetProperty())
		{
			if (FindMethod == ERenderGridFindNodeMethod::Path)
			{
				if (PropertyHandle->GeneratePathToProperty() == PropertyNameOrPath)
				{
					return true;
				}
			}
			else if (PropertyHandle->GetProperty()->GetName() == PropertyNameOrPath)
			{
				return true;
			}
		}
	}

	return false;
}

TSharedPtr<IDetailTreeNode> UE::RenderGrid::Private::RenderGridWidgetUtils::FindTreeNodeRecursive(const TSharedRef<IDetailTreeNode>& RootNode, const FString& PropertyNameOrPath, ERenderGridFindNodeMethod FindMethod)
{
	TArray<TSharedRef<IDetailTreeNode>> Children;
	RootNode->GetChildren(Children);
	for (TSharedRef<IDetailTreeNode>& Child : Children)
	{
		TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(Child, PropertyNameOrPath, FindMethod);
		if (FoundNode.IsValid())
		{
			return FoundNode;
		}
	}

	TSharedPtr<IPropertyHandle> Handle = RootNode->CreatePropertyHandle();
	if (FindPropertyHandleRecursive(Handle, PropertyNameOrPath, FindMethod))
	{
		return RootNode;
	}

	return nullptr;
}

TSharedPtr<IDetailTreeNode> UE::RenderGrid::Private::RenderGridWidgetUtils::FindNode(const TArray<TSharedRef<IDetailTreeNode>>& RootNodes, const FString& QualifiedPropertyName, ERenderGridFindNodeMethod FindMethod)
{
	for (const TSharedRef<IDetailTreeNode>& CategoryNode : RootNodes)
	{
		TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(CategoryNode, QualifiedPropertyName, FindMethod);
		if (FoundNode.IsValid())
		{
			return FoundNode;
		}
	}

	return nullptr;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
