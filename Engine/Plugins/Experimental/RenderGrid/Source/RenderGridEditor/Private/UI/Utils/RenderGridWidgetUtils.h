// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class IDetailTreeNode;
class IPropertyHandle;
class SWidget;


namespace UE::RenderGrid::Private
{
	/**
	 * A static class containing utility functions for widgets.
	 */
	class RenderGridWidgetUtils final
	{
	public:
		enum class ERenderGridFindNodeMethod : uint8
		{
			Name, // Find a node by providing its property name
			Path // Find a node by providing a property path
		};

	public:
		static TSharedRef<SWidget> CreateNodeValueWidget(const TSharedPtr<IDetailTreeNode>& Node);
		static bool FindPropertyHandleRecursive(const TSharedPtr<IPropertyHandle>& PropertyHandle, const FString& PropertyNameOrPath, ERenderGridFindNodeMethod FindMethod);
		static TSharedPtr<IDetailTreeNode> FindTreeNodeRecursive(const TSharedRef<IDetailTreeNode>& RootNode, const FString& PropertyNameOrPath, ERenderGridFindNodeMethod FindMethod);
		static TSharedPtr<IDetailTreeNode> FindNode(const TArray<TSharedRef<IDetailTreeNode>>& RootNodes, const FString& QualifiedPropertyName, ERenderGridFindNodeMethod FindMethod);
	};
}
