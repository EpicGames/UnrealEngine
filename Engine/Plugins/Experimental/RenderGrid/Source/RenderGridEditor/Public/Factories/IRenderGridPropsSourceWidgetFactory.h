// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRenderGridEditor.h"


class URenderGridPropsSourceBase;

namespace UE::RenderGrid::Private
{
	class SRenderGridPropsBase;
}


namespace UE::RenderGrid
{
	/**
	 * The props source widget factory interface.
	 * Implementations should create a widget for the given props source.
	 */
	class RENDERGRIDEDITOR_API IRenderGridPropsSourceWidgetFactory
	{
	public:
		virtual ~IRenderGridPropsSourceWidgetFactory() = default;
		virtual TSharedPtr<Private::SRenderGridPropsBase> CreateInstance(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor) { return nullptr; }
	};
}
