// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintEditor.h"
#include "RenderGrid/RenderGridQueue.h"


class URenderGridQueue;
class URenderGridJob;
class URenderGrid;
class URenderGridBlueprint;

namespace UE::RenderGrid::Private
{
	class FRenderGridBlueprintEditorToolbar;
}


namespace UE::RenderGrid
{
	/**
	 * The render grid editor interface.
	 */
	class IRenderGridEditor : public FBlueprintEditor
	{
	public:
		/** @return The render grid blueprint currently being edited in this editor. */
		virtual URenderGridBlueprint* GetRenderGridBlueprint() const = 0;

		/** @return The render grid instance. */
		virtual URenderGrid* GetInstance() const = 0;

		/** Sets whether the blueprint editor is in debugging mode or not. */
		virtual void SetIsDebugging(const bool bInIsDebugging) = 0;

		/** @return The render grid toolbar. */
		virtual TSharedPtr<Private::FRenderGridBlueprintEditorToolbar> GetRenderGridToolbarBuilder() = 0;

		/** Returns whether it is currently rendering or playing (so changes in the level and such should be ignored). */
		virtual bool IsCurrentlyRenderingOrPlaying() const { return IsBatchRendering() || IsPreviewRendering() || URenderGridQueue::IsRenderingAny() || IsValid(GEditor->PlayWorld); }

		/** Returns whether it can currently render (like a preview render or a batch render) or not. */
		virtual bool CanCurrentlyRender() const { return !IsCurrentlyRenderingOrPlaying(); }

		/** Returns whether it is should currently hide the UI for things to not cause any bugs (like during rendering etc). */
		virtual bool ShouldHideUI() const { return (IsCurrentlyRenderingOrPlaying() && !IsPreviewRendering()); }

		/** Returns whether it is currently batch rendering or not. */
		virtual bool IsBatchRendering() const = 0;

		/** Returns the current batch render queue, or null if it's not currently batch rendering. */
		virtual URenderGridQueue* GetBatchRenderQueue() const = 0;

		/** Returns whether it is currently preview rendering or not. */
		virtual bool IsPreviewRendering() const = 0;

		/** Returns the current preview render queue, or null if it's not currently rendering a preview. */
		virtual URenderGridQueue* GetPreviewRenderQueue() const = 0;

		/** Sets the current preview render queue, set it to null if it's not currently rendering a preview. */
		virtual void SetPreviewRenderQueue(URenderGridQueue* Queue) = 0;

		/** Marks the editing asset as modified. */
		virtual void MarkAsModified() = 0;

		/** Get the currently selected render grid jobs. */
		virtual TArray<URenderGridJob*> GetSelectedRenderGridJobs() const = 0;

		/** Set the selected render grid jobs. */
		virtual void SetSelectedRenderGridJobs(const TArray<URenderGridJob*>& Jobs) = 0;

		DECLARE_MULTICAST_DELEGATE(FOnRenderGridChanged);
		virtual FOnRenderGridChanged& OnRenderGridChanged() { return OnRenderGridChangedDelegate; }

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnRenderGridJobCreated, URenderGridJob*);
		virtual FOnRenderGridJobCreated& OnRenderGridJobCreated() { return OnRenderGridJobCreatedDelegate; }

		DECLARE_MULTICAST_DELEGATE(FOnRenderGridJobsSelectionChanged);
		virtual FOnRenderGridJobsSelectionChanged& OnRenderGridJobsSelectionChanged() { return OnRenderGridJobsSelectionChangedDelegate; }

		DECLARE_MULTICAST_DELEGATE(FOnRenderGridShouldHideUIChanged);
		virtual FOnRenderGridShouldHideUIChanged& OnRenderGridShouldHideUIChanged() { return OnRenderGridShouldHideUIChangedDelegate; }

		DECLARE_MULTICAST_DELEGATE(FOnRenderGridPreviewRenderedFirstTimeSinceAppStart)
		virtual FOnRenderGridPreviewRenderedFirstTimeSinceAppStart& OnRenderGridPreviewRenderedFirstTimeSinceAppStart() { return OnRenderGridPreviewRenderedFirstTimeSinceAppStartDelegate; }

	private:
		/** The delegate for when data in the render grid changed. */
		FOnRenderGridChanged OnRenderGridChangedDelegate;

		/** The delegate for when a render grid job is created. */
		FOnRenderGridJobCreated OnRenderGridJobCreatedDelegate;

		/** The delegate for when the selection of render grid jobs changed. */
		FOnRenderGridJobsSelectionChanged OnRenderGridJobsSelectionChangedDelegate;

		/** The delegate for when the the result of ShouldHideUI() has changed. */
		FOnRenderGridShouldHideUIChanged OnRenderGridShouldHideUIChangedDelegate;

		/** The delegate for the first time (since the start of this application) that the preview is rendered. */
		FOnRenderGridPreviewRenderedFirstTimeSinceAppStart OnRenderGridPreviewRenderedFirstTimeSinceAppStartDelegate;
	};
}
