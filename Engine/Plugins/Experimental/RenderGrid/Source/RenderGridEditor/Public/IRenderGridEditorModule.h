// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IRenderGridEditor.h"
#include "Modules/ModuleManager.h"


class URenderGridBlueprint;
class URenderGridPropsSourceBase;
enum class ERenderGridPropsSourceType : uint8;

namespace UE::RenderGrid
{
	class IRenderGridPropsSourceWidgetFactory;
}

namespace UE::RenderGrid::Private
{
	class SRenderGridPropsBase;
}


namespace UE::RenderGrid
{
	/**
	 * RenderGridEditor module interface.
	 */
	class IRenderGridEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
	{
	public:
		/**
		 * Singleton-like access to IRenderGridEditorModule.
		 *
		 * @return Returns RenderGridEditorModule singleton instance, loading the module on demand if needed.
		 */
		static inline IRenderGridEditorModule& Get()
		{
			return FModuleManager::LoadModuleChecked<IRenderGridEditorModule>(TEXT("RenderGridEditor"));
		}

		/** Creates an instance of the render grid editor. */
		virtual TSharedRef<IRenderGridEditor> CreateRenderGridEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, URenderGridBlueprint* InBlueprint) = 0;

		/** Creates a props source widget for the given props source. */
		virtual TSharedPtr<Private::SRenderGridPropsBase> CreatePropsSourceWidget(URenderGridPropsSourceBase* PropsSource, TSharedPtr<IRenderGridEditor> BlueprintEditor) = 0;

		/** Returns all the factories for creating widgets for props sources. */
		virtual const TMap<ERenderGridPropsSourceType, TSharedPtr<IRenderGridPropsSourceWidgetFactory>>& GetPropsSourceWidgetFactories() const = 0;
	};
}
