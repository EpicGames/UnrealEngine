// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RenderGridEditor : ModuleRules
{
	public RenderGridEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		//PCHUsage = PCHUsageMode.NoPCHs;
		//bUseUnity = false;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RemoteControl",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AppFramework",
				"AssetDefinition",
				"DesktopPlatform",
				"EditorStyle",
				"GraphEditor",
				"InputCore",
				"Kismet",
				"LevelSequence",
				"MovieRenderPipelineCore",
				"MovieScene",
				"Projects",
				"PropertyEditor",
				"RenderGrid",
				"RenderGridDeveloper",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
			}
		);
	}
}